{
	"FileVersion": 3,

	"FriendlyName": "AJA Media Player",
	"Version": 2,
	"VersionName": "2.0",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"MarketplaceURL": "",
	"Description": "Implements input and output using AJA Capture cards.",
	"Category": "Media Players",
	"CanContainContent": true,
	"EnabledByDefault": false,
	"IsBetaVersion": false,
	"SupportedTargetPlatforms": [ "Win64" ],
	"Modules": [
		{
			"Name": "AjaCore",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		},
		{
			"Name": "AjaMedia",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		},
		{
			"Name": "AjaMediaOutput",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		},
		{
			"Name": "AjaMediaFactory",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64" ]
		},
		{
			"Name": "AjaMediaFactory",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64" ]
		},
		{
			"Name": "AjaMediaEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins": [
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		},
		{
			"Name": "MediaPlayerEditor",
			"Enabled": true
		}
	]
}
