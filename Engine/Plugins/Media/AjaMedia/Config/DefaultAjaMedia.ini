[CoreRedirects]

;
; 4.26 Redirects
;

; Axis Switch processors
+PropertyRedirects=(OldName="AjaTimecodeProvider.ReferenceConfiguration",NewName="AjaTimecodeProvider.LTCConfiguration")