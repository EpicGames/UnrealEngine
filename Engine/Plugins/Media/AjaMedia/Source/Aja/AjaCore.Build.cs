// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class AjaCore : ModuleRules
{
	public AjaCore(ReadOnlyTargetRules Target) : base(Target)
	{
		string EngineDir = Path.GetFullPath(Target.RelativeEnginePath);
		string AjaPluginDirectory = Path.Combine(EngineDir, "Plugins", "Media", "AjaMedia");
		string AjaThirdPartyPath = Path.Combine(AjaPluginDirectory, "Source", "ThirdParty", "AjaLib");

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"GPUTextureTransfer",
				"RenderCore"
			});
			
		
		PrivateIncludePaths.AddRange(
			new string[] {
				Path.Combine(AjaThirdPartyPath, "ntv2lib-deploy", "includes"),
				Path.Combine(AjaThirdPartyPath, "ntv2lib-deploy", "includes", "ajantv2", "includes"),
				Path.Combine(AjaThirdPartyPath, "ntv2lib-deploy", "includes", "ajantv2", "src", "win")
			});
			
			
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"RHI"
			});

		bDisableStaticAnalysis = true;

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			string LibName = "libajantv2.lib";

			if (Target.Configuration == UnrealTargetConfiguration.Debug)
			{
				// Disable for now to fix clang build, to reproduce: compile in Debug Editor with -stresstestunity -nodebuginfo flags
				//LibName = "libajantv2d.lib";
			}

			string LibPath = Path.Combine(AjaThirdPartyPath, "ntv2lib-deploy", "lib", Target.Architecture.WindowsLibDir, LibName);
			PublicAdditionalLibraries.Add(LibPath);
			PublicSystemLibraries.Add("shlwapi.lib");
			PublicSystemLibraries.Add("wbemuuid.lib");
			PublicDefinitions.Add("AJAMEDIA_DLL_PLATFORM=1");
		}
		else
		{
			PublicDefinitions.Add("AJAMEDIA_DLL_PLATFORM=0");
			System.Console.WriteLine("AJA not supported on this platform");
		}
	}
}
