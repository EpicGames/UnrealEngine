// Copyright Epic Games, Inc. All Rights Reserved.

#define MSWindows 1
#define AJA_WINDOWS 1
#define AJA_NO_AUTOIMPORT 1

#include "AJALib.h"

#include "GPUTextureTransfer.h"

#include "AutoDetectChannel.h"
#include "DeviceScanner.h"
#include "InputChannel.h"
#include "Misc/Timecode.h"
#include "OutputChannel.h"
#include "SyncChannel.h"
#include "TimecodeChannel.h"
#include "VideoFormats.h"
#include "Helpers.h"

#include <cwchar>

namespace AJA
{

	/*
	 * Timecode
	 */

	FTimecode::FTimecode()
		: Hours(0)
		, Minutes(0)
		, Seconds(0)
		, Frames(0)
		, bDropFrame(false)
	{}

	bool FTimecode::operator== (const FTimecode& Other) const
	{
		return Other.Hours == Hours
			&& Other.Minutes == Minutes
			&& Other.Seconds == Seconds
			&& Other.Frames == Frames;
	}

	/*
	 * String Helpers
	 */

	 // unicode to wide
	std::wstring s2ws(const std::string& str)
	{
		int size_needed = MultiByteToWideChar(CP_UTF8, 0, &str[0], (int)str.size(), NULL, 0) + 1;
		std::wstring wstrTo(size_needed, 0);
		MultiByteToWideChar(CP_UTF8, 0, &str[0], (int)str.size(), &wstrTo[0], size_needed);
		return wstrTo;
	}

	// wide to unicode
	std::string ws2s(const std::wstring& wstr)
	{
		int size_needed = WideCharToMultiByte(CP_ACP, 0, wstr.c_str(), int(wstr.length() + 1), 0, 0, 0, 0);
		std::string strTo(size_needed, 0);
		WideCharToMultiByte(CP_ACP, 0, wstr.c_str(), int(wstr.length() + 1), &strTo[0], size_needed, 0, 0);
		return strTo;
	}

	/* AJAVideoFrameFormats implementation
	*****************************************************************************/
	AJADeviceScanner::AJADeviceScanner()
	{
		Scanner = new AJA::Private::DeviceScanner();
	}

	AJADeviceScanner::~AJADeviceScanner()
	{
		delete Scanner;
	}

	int32_t AJADeviceScanner::GetNumDevices() const
	{
		return Scanner->GetNumDevices();
	}

	bool AJADeviceScanner::GetDeviceTextId(int32_t InDeviceIndex, AJADeviceScanner::FormatedTextType& OutTextId) const
	{
		return Scanner->GetDeviceTextId(InDeviceIndex, OutTextId);
	}

	bool AJADeviceScanner::GetDeviceInfo(int32_t InDeviceIndex, DeviceInfo& OutDeviceInfo) const
	{
		return Scanner->GetDeviceInfo(InDeviceIndex, OutDeviceInfo);
	}

	/* AJAVideoFrameFormats implementation
	*****************************************************************************/
	AJAVideoFormats::VideoFormatDescriptor::VideoFormatDescriptor()
		: bIsValid(false)
	{}

	AJAVideoFormats::AJAVideoFormats(int32_t InDeviceId)
	{
		Formats = new AJA::Private::VideoFormatsScanner(InDeviceId);
	}

	AJAVideoFormats::~AJAVideoFormats()
	{
		delete Formats;
	}

	int32_t AJAVideoFormats::GetNumSupportedFormat() const
	{
		return (int32_t)Formats->FormatList.size();
	}

	AJAVideoFormats::VideoFormatDescriptor AJAVideoFormats::GetSupportedFormat(int32_t InIndex) const
	{
		if (InIndex < Formats->FormatList.size())
		{
			return Formats->FormatList[InIndex];
		}
		return AJAVideoFormats::VideoFormatDescriptor();
	}

	AJAVideoFormats::VideoFormatDescriptor AJAVideoFormats::GetVideoFormat(FAJAVideoFormat InVideoFormatIndex)
	{
		return Private::VideoFormatsScanner::GetVideoFormat(InVideoFormatIndex);
	}

	std::string AJAVideoFormats::VideoFormatToString(FAJAVideoFormat InVideoFormatIndex)
	{
		NTV2VideoFormat NativeVideoFormat;

		if (!AJA::Private::Helpers::TryVideoFormatIndexToNTV2VideoFormat(InVideoFormatIndex, NativeVideoFormat))
		{
			return "";
		}

		return NTV2VideoFormatToString(NativeVideoFormat);
	}

	bool AJAVideoFormats::VideoFormatToString(FAJAVideoFormat InVideoFormatIndex, char* OutCStr, uint32_t MaxLen)
	{
		if (!OutCStr || (MaxLen < 1))
		{
			return false;
		}

		std::string VideoFormatStdStr = VideoFormatToString(InVideoFormatIndex);

		if (!VideoFormatStdStr.length() || (VideoFormatStdStr.length() >= MaxLen))
		{
			OutCStr[0] = '\0';
			return false;
		}

		return !strcpy_s(OutCStr, MaxLen, VideoFormatStdStr.c_str());
	}

	/* IAJASyncChannelCallbackInterface implementation
	*****************************************************************************/
	IAJASyncChannelCallbackInterface::IAJASyncChannelCallbackInterface()
	{}

	IAJASyncChannelCallbackInterface::~IAJASyncChannelCallbackInterface()
	{}

	/* AJASyncChannelOptions implementation
	*****************************************************************************/
	AJASyncChannelOptions::AJASyncChannelOptions(const TCHAR* DebugName)
		: CallbackInterface(nullptr)
		, TransportType(ETransportType::TT_SdiSingle)
		, ChannelIndex(-1)
		, TimecodeFormat(AJA::ETimecodeFormat::TCF_LTC)
		, bOutput(false)
		, bWaitForFrameToBeReady(false)
		, bAutoDetectFormat(false)
	{
	}

	/* AJASyncChannel implementation
	*****************************************************************************/
	AJASyncChannel::AJASyncChannel()
	{
		Channel = new Private::SyncChannel();
	}

	bool AJASyncChannel::Initialize(const AJADeviceOptions& InDevice, const AJASyncChannelOptions& InOption)
	{
		return Channel->Initialize(InDevice, InOption);
	}

	void AJASyncChannel::Uninitialize()
	{
		Channel->Uninitialize();
	}

	AJASyncChannel::~AJASyncChannel()
	{
		delete Channel;
	}

	bool AJASyncChannel::WaitForSync() const
	{
		return Channel ? Channel->WaitForSync() : false;
	}

	bool AJASyncChannel::GetTimecode(FTimecode& OutTimecode) const
	{
		return Channel ? Channel->GetTimecode(OutTimecode) : false;
	}

	bool AJASyncChannel::GetSyncCount(uint32_t& OutCount) const
	{
		return Channel ? Channel->GetSyncCount(OutCount) : false;
	}

	bool AJASyncChannel::GetVideoFormat(FAJAVideoFormat& OutVideoFormat)
	{
		if (!Channel)
		{
			return false;
		}

		NTV2VideoFormat NativeVideoFormat;
		const bool bValidFormat = Channel->GetVideoFormat(NativeVideoFormat);
		OutVideoFormat = FAJAVideoFormat(NativeVideoFormat);

		return bValidFormat;
	}

	/* IAJATimecodeChannelCallbackInterface implementation
	*****************************************************************************/
	IAJATimecodeChannelCallbackInterface::IAJATimecodeChannelCallbackInterface()
	{}

	IAJATimecodeChannelCallbackInterface::~IAJATimecodeChannelCallbackInterface()
	{}

	/* AJATimecodeChannelOptions implementation
	*****************************************************************************/
	AJATimecodeChannelOptions::AJATimecodeChannelOptions(const TCHAR* DebugName)
		: CallbackInterface(nullptr)
		, bUseDedicatedPin(false)
		, bReadTimecodeFromReferenceIn(false)
		, LTCSourceIndex(1)
		, LTCFrameRateNumerator(0)
		, LTCFrameRateDenominator(1)
		, TransportType(ETransportType::TT_SdiSingle)
		, ChannelIndex(-1)
		, TimecodeFormat(AJA::ETimecodeFormat::TCF_LTC)
		, bAutoDetectFormat(false)
	{
	}

	/* AJASyncChannel implementation
	*****************************************************************************/
	AJATimecodeChannel::AJATimecodeChannel()
	{
		Channel = new Private::TimecodeChannel();
	}

	bool AJATimecodeChannel::Initialize(const AJADeviceOptions& InDevice, const AJATimecodeChannelOptions& InOption)
	{
		return Channel->Initialize(InDevice, InOption);
	}

	void AJATimecodeChannel::Uninitialize()
	{
		Channel->Uninitialize();
	}

	AJATimecodeChannel::~AJATimecodeChannel()
	{
		delete Channel;
	}

	bool AJATimecodeChannel::GetTimecode(FTimecode& OutTimecode) const
	{
		return Channel ? Channel->GetTimecode(OutTimecode) : false;
	}

	/* AJAInputFrameData implementation
	*****************************************************************************/
	AJAInputFrameData::AJAInputFrameData()
		:FramesDropped(0)
	{}

	/* AJAOutputFrameData implementation
	*****************************************************************************/
	AJAOutputFrameData::AJAOutputFrameData()
		: FramesLost(0)
	{}

	/* AJAAncillaryFrameData implementation
	*****************************************************************************/
	AJAAncillaryFrameData::AJAAncillaryFrameData()
	{
		memset(this, 0, sizeof(AJAAncillaryFrameData));
	}

	/* AJAAudioFrameData implementation
	*****************************************************************************/
	AJAAudioFrameData::AJAAudioFrameData()
	{
		memset(this, 0, sizeof(AJAAudioFrameData));
	}

	/* AJAVideoFrameData implementation
	*****************************************************************************/
	AJAVideoFrameData::AJAVideoFrameData()
	{
		memset(this, 0, sizeof(AJAVideoFrameData));
	}

	/* AJARequestInputBufferData implementation
	*****************************************************************************/
	AJARequestInputBufferData::AJARequestInputBufferData()
	{
		memset(this, 0, sizeof(AJARequestInputBufferData));
	}

	/* AJARequestedInputBufferData implementation
	*****************************************************************************/
	AJARequestedInputBufferData::AJARequestedInputBufferData()
	{
		memset(this, 0, sizeof(AJARequestedInputBufferData));
	}

	/* IAJAInputOutputChannelCallbackInterface implementation
	*****************************************************************************/
	IAJAInputOutputChannelCallbackInterface::IAJAInputOutputChannelCallbackInterface()
	{
	}

	/* AJAInputOutputChannelOptions implementation
	*****************************************************************************/
	AJAInputOutputChannelOptions::AJAInputOutputChannelOptions(const TCHAR* DebugName, uint32_t InChannelIndex)
		: CallbackInterface(nullptr)
		, NumberOfAudioChannel(8)
		, TransportType(ETransportType::TT_SdiSingle)
		, ChannelIndex(InChannelIndex)
		, SynchronizeChannelIndex(InChannelIndex)
		, KeyChannelIndex(InChannelIndex)
		, OutputNumberOfBuffers(2)
		, VideoFormatIndex(NTV2VideoFormat::NTV2_FORMAT_1080p_3000)
		, PixelFormat(AJA::EPixelFormat::PF_8BIT_YCBCR)
		, TimecodeFormat(AJA::ETimecodeFormat::TCF_LTC)
		, OutputReferenceType(EAJAReferenceType::EAJA_REFERENCETYPE_FREERUN)
		, Options(0)
	{
		bUseVideo = true;
		bDisplayWarningIfDropFrames = true;
	}

	/* AJAInputChannel implementation
	*****************************************************************************/
	AJAInputChannel::AJAInputChannel()
	{
		Channel = new Private::InputChannel();
	}

	bool AJAInputChannel::Initialize(const AJADeviceOptions& InDevice, const AJAInputOutputChannelOptions& Options)
	{
		return Channel->Initialize(InDevice, Options);
	}

	void AJAInputChannel::Uninitialize()
	{
		Channel->Uninitialize();
	}

	AJAInputChannel::~AJAInputChannel()
	{
		delete Channel;
	}

	uint32_t AJAInputChannel::GetFrameDropCount() const
	{
		return Channel ? Channel->GetCurrentAutoCirculateStatus().GetDroppedFrameCount() : 0;
	}

	const AJAInputOutputChannelOptions& AJAInputChannel::GetOptions() const
	{
		static AJAInputOutputChannelOptions InvalidOptions(TEXT("InvalidOptions"), 0);

		return Channel ? Channel->GetOptions() : InvalidOptions;
	}

	const AJADeviceOptions& AJAInputChannel::GetDeviceOptions() const
	{
		static AJADeviceOptions InvalidOptions(0);
		return Channel ? Channel->GetDeviceOptions() : InvalidOptions;
	}

	/* AJAOutputFrameBufferData implementation
	*****************************************************************************/
	const uint32_t AJAOutputFrameBufferData::InvalidFrameIdentifier = static_cast<uint32_t>(-1);

	AJAOutputFrameBufferData::AJAOutputFrameBufferData()
		: FrameIdentifier(InvalidFrameIdentifier)
		, bEvenFrame(false)
	{
	}

	/* AJAOutputChannel implementation
	*****************************************************************************/
	AJAOutputChannel::AJAOutputChannel()
	{
		Channel = new Private::OutputChannel();
	}

	bool AJAOutputChannel::Initialize(const AJADeviceOptions& InDevice, const AJAInputOutputChannelOptions& Options)
	{
		return Channel->Initialize(InDevice, Options);
	}

	void AJAOutputChannel::Uninitialize(TPromise<void> Promise)
	{
		Channel->Uninitialize(MoveTemp(Promise));
	}

	AJAOutputChannel::~AJAOutputChannel()
	{
		delete Channel;
	}

	bool AJAOutputChannel::SetAncillaryFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* AncillaryBuffer, uint32_t AncillaryBufferSize)
	{
		return Channel ? Channel->SetAncillaryFrameData(InFrameData, AncillaryBuffer, AncillaryBufferSize) : false;
	}

	bool AJAOutputChannel::SetAudioFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* AudioBuffer, uint32_t AudioBufferSize)
	{
		return Channel ? Channel->SetAudioFrameData(InFrameData, AudioBuffer, AudioBufferSize) : false;
	}

	bool AJAOutputChannel::SetVideoFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* VideoBuffer, uint32_t VideoBufferSize)
	{
		return Channel ? Channel->SetVideoFrameData(InFrameData, VideoBuffer, VideoBufferSize) : false;
	}

	bool AJAOutputChannel::SetVideoFrameData(const AJAOutputFrameBufferData& InFrameData, FRHITexture* RHITexture)
	{
		return Channel ? Channel->SetVideoFrameData(InFrameData, RHITexture) : false;
	}

	bool AJAOutputChannel::DMAWriteAudio(const uint8* Buffer, int32 BufferSize)
	{
		return Channel ? Channel->DMAWriteAudio(Buffer, BufferSize) : false;
	}

	bool AJAOutputChannel::ShouldCaptureThisFrame(::FTimecode Timecode, uint32 SourceFrameNumber)
	{
		return Channel ? Channel->ShouldCaptureThisFrame(Timecode, SourceFrameNumber) : false;
	}

	bool AJAOutputChannel::GetOutputDimension(uint32_t& OutWidth, uint32_t& OutHeight) const
	{
		return Channel ? Channel->GetOutputDimension(OutWidth, OutHeight) : false;
	}

	int32_t AJAOutputChannel::GetNumAudioSamplesPerFrame(const AJAOutputFrameBufferData& InFrameData) const
	{
		return Channel ? Channel->GetNumAudioSamplesPerFrame(InFrameData) : 0;
	}

	/* AJAAutoDetectChannel implementation
	*****************************************************************************/
	IAJAAutoDetectCallbackInterface::IAJAAutoDetectCallbackInterface()
	{
	}

	IAJAAutoDetectCallbackInterface::~IAJAAutoDetectCallbackInterface()
	{
	}

	AJAAutoDetectChannel::AutoDetectChannelData::AutoDetectChannelData()
		: DetectedVideoFormat(0)
		, ChannelIndex(0)
	{}

	AJAAutoDetectChannel::AJAAutoDetectChannel()
	{
		AutoChannel = new AJA::Private::AutoDetectChannel();
	}

	AJAAutoDetectChannel::~AJAAutoDetectChannel()
	{
		delete AutoChannel;
	}

	bool AJAAutoDetectChannel::Initialize(IAJAAutoDetectCallbackInterface* InCallbackInterface)
	{
		return AutoChannel->Initialize(InCallbackInterface);
	}

	void AJAAutoDetectChannel::Uninitialize()
	{
		AutoChannel->Uninitialize();
	}

	int32_t AJAAutoDetectChannel::GetNumOfChannelData() const
	{
		return (int32_t)AutoChannel->GetFoundChannels().size();
	}

	AJAAutoDetectChannel::AutoDetectChannelData AJAAutoDetectChannel::GetChannelData(int32_t InIndex) const
	{
		std::vector<AJAAutoDetectChannel::AutoDetectChannelData> FoundChannel = AutoChannel->GetFoundChannels();
		if (InIndex < FoundChannel.size() && InIndex >= 0)
		{
			return FoundChannel[InIndex];
		}
		return AJAAutoDetectChannel::AutoDetectChannelData();
	}
}