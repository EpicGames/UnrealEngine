// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoDetectChannel.h"
#include "Helpers.h"
#include "DeviceScanner.h"

namespace AJA
{
	namespace Private
	{
		/* AutoDetectChannel implementation
		*****************************************************************************/
		bool AutoDetectChannel::Initialize(IAJAAutoDetectCallbackInterface* InCallbackInterface)
		{
			Uninitialize();
			AutoDetectImplementation.reset(new AutoDetectChannelImpl(InCallbackInterface));

			bool bResult = AutoDetectImplementation->CanInitialize();
			if (bResult)
			{
				// Get the first device. It doesn't matter which one since we want only want the device to detect the input for other device.
				AJADeviceOptions DeviceOption = AJADeviceOptions(0);

				Device = DeviceCache::GetDevice(DeviceOption);
				std::shared_ptr<AutoDetectChannelInitialize_DeviceCommand> SharedCommand(new AutoDetectChannelInitialize_DeviceCommand(Device, AutoDetectImplementation));
				InitializeCommand = SharedCommand;
				Device->AddCommand(std::static_pointer_cast<DeviceCommand>(SharedCommand));
			}

			return bResult;
		}

		void AutoDetectChannel::Uninitialize()
		{
			if (AutoDetectImplementation)
			{
				std::shared_ptr<AutoDetectChannelInitialize_DeviceCommand> SharedCommand = InitializeCommand.lock();
				if (SharedCommand && Device)
				{
					Device->RemoveCommand(std::static_pointer_cast<DeviceCommand>(SharedCommand));
				}

				AutoDetectImplementation->Uninitialize(); // delete is done in AutoDetectChannelUninitialize_DeviceCommand completed

				if (Device)
				{
					std::shared_ptr<AutoDetectChannelUninitialize_DeviceCommand> UninitializeCommand;
					UninitializeCommand.reset(new AutoDetectChannelUninitialize_DeviceCommand(Device, AutoDetectImplementation));
					Device->AddCommand(std::static_pointer_cast<DeviceCommand>(UninitializeCommand));
				}
			}

			InitializeCommand.reset();
			AutoDetectImplementation.reset();
			Device.reset();
		}

		std::vector<AJAAutoDetectChannel::AutoDetectChannelData> AutoDetectChannel::GetFoundChannels() const
		{
			if (AutoDetectImplementation)
			{
				return AutoDetectImplementation->GetFoundChannels();
			}
			return std::vector<AJAAutoDetectChannel::AutoDetectChannelData>();
		}

		/* AutoDetectChannelInitialize_DeviceCommand implementation
		*****************************************************************************/
		AutoDetectChannelInitialize_DeviceCommand::AutoDetectChannelInitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<AutoDetectChannelImpl>& InChannel)
			: DeviceCommand(InConnection)
			, AutoDetectImplementation(InChannel)
		{}

		void AutoDetectChannelInitialize_DeviceCommand::Execute(DeviceConnection::CommandList& InCommandList)
		{
			std::shared_ptr<AutoDetectChannelImpl> Shared = AutoDetectImplementation.lock();
			if (Shared) // could have been Uninitialize before we have time to execute it
			{
				if (!Shared->Thread_Initialize(InCommandList))
				{
					Shared->Thread_Destroy(InCommandList);
				}
			}
			AutoDetectImplementation.reset();
		}

		/* AutoDetectChannelUninitialize_DeviceCommand implementation
		*****************************************************************************/
		AutoDetectChannelUninitialize_DeviceCommand::AutoDetectChannelUninitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<AutoDetectChannelImpl>& InChannel)
			: DeviceCommand(InConnection)
			, AutoDetectImplementation(InChannel)
		{}

		void AutoDetectChannelUninitialize_DeviceCommand::Execute(DeviceConnection::CommandList& InCommandList)
		{
			// keep a shared pointer to prevent the destruction of the SyncChannel until its turn comes up
			AutoDetectImplementation->Thread_Destroy(InCommandList);
			AutoDetectImplementation.reset();
		}

		/* AutoDetectChannelImpl implementation
		*****************************************************************************/
		AutoDetectChannelImpl::AutoDetectChannelImpl(IAJAAutoDetectCallbackInterface* InCallbackInterface)
			: CallbackInterface(InCallbackInterface)
			//, Card(nullptr)
			, bStopRequested(false)
			, bProcessCompleted(false)
		{
		}

		AutoDetectChannelImpl::~AutoDetectChannelImpl()
		{
			//assert(Card == nullptr);
		}

		bool AutoDetectChannelImpl::CanInitialize() const
		{
			return true;
		}

		bool AutoDetectChannelImpl::Thread_Initialize(DeviceConnection::CommandList& InCommandList)
		{
			bool bResult = Thread_Configure(InCommandList);
			bProcessCompleted = true;
			Thread_OnInitializationCompleted(bResult);
			return bResult;
		}

		void AutoDetectChannelImpl::Uninitialize()
		{
			AJAAutoLock AutoLock(&Lock);
			CallbackInterface = nullptr;
			bStopRequested = true;
		}

		void AutoDetectChannelImpl::Thread_Destroy(DeviceConnection::CommandList& InCommandList)
		{
		}

		std::vector<AJAAutoDetectChannel::AutoDetectChannelData> AutoDetectChannelImpl::GetFoundChannels() const
		{
			if (bProcessCompleted)
			{
				return FoundChannels;
			}
			return std::vector<AJAAutoDetectChannel::AutoDetectChannelData>();
		}

		void AutoDetectChannelImpl::Thread_OnInitializationCompleted(bool bSucceed)
		{
			AJAAutoLock AutoLock(&Lock);
			if (CallbackInterface && !bStopRequested)
			{
				CallbackInterface->OnCompletion(bSucceed);
			}
		}

		bool AutoDetectChannelImpl::Thread_Configure(DeviceConnection::CommandList& InCommandList)
		{
			if (bStopRequested)
			{
				return false;
			}

			if (!CanInitialize())
			{
				return false;
			}

			if (bStopRequested)
			{
				return false;
			}

			std::unique_ptr<DeviceScanner> Scanner = std::make_unique<DeviceScanner>();

			const int32_t NumberOfDevices = Scanner->GetNumDevices();
			for (int32_t DeviceIndex = 0; DeviceIndex < NumberOfDevices; ++DeviceIndex)
			{
				if (bStopRequested)
				{
					return false;
				}

				AJADeviceScanner::DeviceInfo DeviceInfo;
				bool bValidDeviceInfo = Scanner->GetDeviceInfo(DeviceIndex, DeviceInfo);
				bValidDeviceInfo = bValidDeviceInfo && DeviceInfo.bIsSupported;

				if (!bValidDeviceInfo)
				{
					UE_LOGF(LogTemp,  Display, "AutoDetectChannel: Device %d is not valid.\n", DeviceIndex);
					break;
				}

				if (!Thread_Detect(DeviceIndex, DeviceInfo))
				{
					break;
				}
			}

			return !bStopRequested;
		}

		bool AutoDetectChannelImpl::Thread_Detect(int32_t InDeviceIndex, const AJADeviceScanner::DeviceInfo& InDeviceInfo)
		{
			std::unique_ptr<CNTV2Card> NewCard = std::make_unique<CNTV2Card>(InDeviceIndex);

			uint32_t Counter = 0;
			while (!NewCard->IsDeviceReady())
			{
				const DWORD SleepMilli = 10;
				const DWORD TimeoutMilli = 2000;
				if (Counter * SleepMilli > TimeoutMilli)
				{
					UE_LOGF(LogTemp,  Display, "AutoDetectChannel: Can't get the %d device initialized.\n", InDeviceIndex);
					return false;
				}
				++Counter;
				::Sleep(SleepMilli);
			}

			// save the service level.
			NTV2EveryFrameTaskMode TaskMode;
			NewCard->GetEveryFrameServices(TaskMode);
			// Use the OEM service level.
			NewCard->SetEveryFrameServices(NTV2_OEM_TASKS);

			::Sleep(50 * 1); // Wait 1 frame

			const bool bIsSDI = true;
			const bool bHasBiDirectionalSDI = ::NTV2DeviceHasBiDirectionalSDI(NewCard->GetDeviceID());

			std::vector<NTV2Channel> ChannelsToEnable;

			for (int32_t InputSourceIndex = 0; InputSourceIndex < InDeviceInfo.NumSdiInput; ++InputSourceIndex)
			{
				const NTV2Channel Channel = NTV2Channel(InputSourceIndex);

				bool bChannelEnabled = false;
				AJA_CHECK(NewCard->IsChannelEnabled(Channel, bChannelEnabled));
				if (!bChannelEnabled)
				{
					ChannelsToEnable.push_back(Channel);
					AJA_CHECK(NewCard->EnableChannel(Channel));
				}

				if (bHasBiDirectionalSDI)
				{
					AJA_CHECK(NewCard->SetSDITransmitEnable(Channel, false));
				}
			}

			::Sleep(50 * 12); // Wait 12 frames

			for (int32_t InputSourceIndex = 0; InputSourceIndex < InDeviceInfo.NumSdiInput; ++InputSourceIndex)
			{
				const NTV2Channel Channel = NTV2Channel(InputSourceIndex);
				const NTV2InputSource InputSource = bIsSDI ? GetNTV2InputSourceForIndex(InputSourceIndex) : GetNTV2HDMIInputSourceForIndex(InputSourceIndex);

				bool bIsProgressive = true;
				NTV2VideoFormat FoundVideoFormat = NewCard->GetInputVideoFormat(InputSource, bIsProgressive);

				if (FoundVideoFormat != NTV2_FORMAT_UNKNOWN)
				{
					AJAAutoDetectChannel::AutoDetectChannelData AutoData;
					AutoData.DeviceIndex = InDeviceIndex;
					AutoData.ChannelIndex = InputSourceIndex;
					AutoData.DetectedVideoFormat = FoundVideoFormat;
					AutoData.TimecodeFormat = Helpers::GetTimecodeFormat(NewCard.get(), Channel);

					FoundChannels.push_back(AutoData);
				}
			}

			for (NTV2Channel Channel : ChannelsToEnable)
			{
				AJA_CHECK(NewCard->DisableChannel(Channel));
			}

			NewCard->SetEveryFrameServices(TaskMode);

			return true;
		}

	}
}
