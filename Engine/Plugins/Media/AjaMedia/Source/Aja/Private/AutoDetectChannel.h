// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Device.h"
#include "Helpers.h"

#include <vector>


namespace AJA
{
	namespace Private
	{
		class AutoDetectChannel;
		class AutoDetectChannelImpl;
		struct AutoDetectChannelInitialize_DeviceCommand;
		struct AutoDetectChannelUninitialize_DeviceCommand;

		/* AutoDetectChannel definition
		*****************************************************************************/
		class AutoDetectChannel
		{
		public:
			AutoDetectChannel() = default;
			AutoDetectChannel(const AutoDetectChannel&) = delete;
			AutoDetectChannel& operator=(const AutoDetectChannel&) = delete;

			bool Initialize(IAJAAutoDetectCallbackInterface* InCallbackInterface);
			void Uninitialize();

			std::vector<AJAAutoDetectChannel::AutoDetectChannelData> GetFoundChannels() const;

		private:
			std::shared_ptr<AutoDetectChannelImpl> AutoDetectImplementation;
			std::shared_ptr<DeviceConnection> Device;
			std::weak_ptr<AutoDetectChannelInitialize_DeviceCommand> InitializeCommand;
		};


		/* AutoDetectChannelInitialize_DeviceCommand definition
		*****************************************************************************/
		struct AutoDetectChannelInitialize_DeviceCommand : DeviceCommand
		{
			AutoDetectChannelInitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<AutoDetectChannelImpl>& InChannel);

			virtual void Execute(DeviceConnection::CommandList& InCommandList) override;

			std::weak_ptr<AutoDetectChannelImpl> AutoDetectImplementation;
		};


		/* AutoDetectChannelUninitialize_DeviceCommand definition
		*****************************************************************************/
		struct AutoDetectChannelUninitialize_DeviceCommand : DeviceCommand
		{
			AutoDetectChannelUninitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<AutoDetectChannelImpl>& InChannel);

			virtual void Execute(DeviceConnection::CommandList& InCommandList) override;

			std::shared_ptr<AutoDetectChannelImpl> AutoDetectImplementation;
		};


		/* AutoDetectChannelImpl definition
		*****************************************************************************/
		class AutoDetectChannelImpl
		{
		public:
			AutoDetectChannelImpl() = delete;
			AutoDetectChannelImpl(IAJAAutoDetectCallbackInterface* InCallbackInterface);
			AutoDetectChannelImpl(const AutoDetectChannelImpl&) = delete;
			AutoDetectChannelImpl& operator=(const AutoDetectChannelImpl&) = delete;
			virtual ~AutoDetectChannelImpl();

			bool CanInitialize() const;
			bool Thread_Initialize(DeviceConnection::CommandList& InCommandList);
			void Uninitialize();
			void Thread_Destroy(DeviceConnection::CommandList& InCommandList);

			std::vector<AJAAutoDetectChannel::AutoDetectChannelData> GetFoundChannels() const;

		private:
			AJALock Lock; // for callback
			IAJAAutoDetectCallbackInterface* CallbackInterface;

			std::vector<AJAAutoDetectChannel::AutoDetectChannelData> FoundChannels;

			volatile bool bStopRequested;
			volatile bool bProcessCompleted;

		private:
			bool Thread_Configure(DeviceConnection::CommandList& InCommandList);
			void Thread_OnInitializationCompleted(bool bSucceed);
			bool Thread_Detect(int32_t InDeviceIndex, const AJADeviceScanner::DeviceInfo& InDeviceInfo);
		};
	}
}
