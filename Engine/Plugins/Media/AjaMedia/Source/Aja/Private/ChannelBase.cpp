// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChannelBase.h"

namespace AJA
{
	namespace Private
	{
		/* IOChannelInitialize_DeviceCommand implementation
		*****************************************************************************/
		IOChannelInitialize_DeviceCommand::IOChannelInitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<ChannelThreadBase>& InChannel)
			: DeviceCommand(InConnection)
			, ChannelThread(InChannel)
		{}

		void IOChannelInitialize_DeviceCommand::Execute(DeviceConnection::CommandList& InCommandList)
		{
			std::shared_ptr<ChannelThreadBase> Shared = ChannelThread.lock();
			if (Shared) // could have been Uninitialize before we have time to execute it
			{
				if (!Shared->DeviceThread_Initialize(InCommandList))
				{
					Shared->DeviceThread_Destroy(InCommandList);
				}
			}
			ChannelThread.reset();
		}

		/* IOChannelUninitialize_DeviceCommand implementation
		*****************************************************************************/
		IOChannelUninitialize_DeviceCommand::IOChannelUninitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<ChannelThreadBase>& InChannel, TPromise<void> InCompletionPromise)
			: DeviceCommand(InConnection)
			, ChannelThread(InChannel)
			, CompletionPromise(MoveTemp(InCompletionPromise))
		{}

		void IOChannelUninitialize_DeviceCommand::Execute(DeviceConnection::CommandList& InCommandList)
		{
			// keep a shared pointer to prevent the destruction of the Channel until its turn comes up

			// wait for thread to be done
			ChannelThread->FrameThread.Stop();
			ChannelThread->DeviceThread_Destroy(InCommandList);
			ChannelThread.reset();
			CompletionPromise.SetValue();
		}

		/* ChannelThreadBase implementation
		*****************************************************************************/

		ChannelThreadBase::ChannelThreadBase(const AJADeviceOptions& InDevice, const AJAInputOutputChannelOptions& InOptions)
			: DeviceOption(InDevice)
			//, Card(nullptr)
			, Channel(NTV2_CHANNEL_INVALID)
			, KeyChannel(NTV2_CHANNEL_INVALID)
			, InputSource(NTV2_INPUTSOURCE_INVALID)
			, KeySource(NTV2_INPUTSOURCE_INVALID)
			, AudioSystem(NTV2_AUDIOSYSTEM_INVALID)
			, FormatDescriptor(NTV2_STANDARD_1080p, NTV2_FBF_8BIT_YCBCR)
			, TimecodeBurner(nullptr)
			, BaseFrameIndex(0)
			, AncBufferSize(0)
			, AncF2BufferSize(0)
			, AudioBufferSize(0)
			, VideoBufferSize(0)
			, bRegisteredChannel(false)
			, bRegisteredKeyChannel(false)
			, bRegisteredReference(false)
			, bConnectedChannel(false)
			, bConnectedKeyChannel(false)
			, bStopRequested(false)
			, Options(InOptions)
		{
			if (Options.ChannelIndex > NTV2_MAX_NUM_CHANNELS || Options.ChannelIndex < 1)
			{
				UE_LOGF(LogAjaCore, Error, "AJAChannel: The port index '%d' is invalid.\n", Options.ChannelIndex);
			}
			else
			{
				Channel = (NTV2Channel)(Options.ChannelIndex - 1);
				InputSource = GetNTV2InputSourceForIndex(Channel, Helpers::IsHdmiTransport(Options.TransportType) ? NTV2_INPUTSOURCES_HDMI : NTV2_INPUTSOURCES_SDI);
			}

			if (UseKey())
			{
				if (Options.KeyChannelIndex > NTV2_MAX_NUM_CHANNELS || Options.KeyChannelIndex < 1)
				{
					UE_LOGF(LogAjaCore, Error, "AJAChannel: The key port index '%d' is invalid.\n", Options.KeyChannelIndex);
					Options.bUseKey = false;
				}
				else if (Options.KeyChannelIndex == Options.ChannelIndex)
				{
					UE_LOGF(LogAjaCore, Error, "AJAChannel: The key port index '%d' is the same as the port index '%d'.\n", Options.KeyChannelIndex, Options.ChannelIndex);
					Options.bUseKey = false;
				}
				else
				{
					KeyChannel = (NTV2Channel)(Options.KeyChannelIndex - 1);
					KeySource = GetNTV2InputSourceForIndex(KeyChannel, Helpers::IsHdmiTransport(Options.TransportType) ? NTV2_INPUTSOURCES_HDMI : NTV2_INPUTSOURCES_SDI);
				}
			}
		}
		
		ChannelThreadBase::~ChannelThreadBase()
		{
			//assert(Card == nullptr);
			assert(Device == nullptr);
		}

		AUTOCIRCULATE_STATUS ChannelThreadBase::GetCurrentAutoCirculateStatus() const
		{
			AUTOCIRCULATE_STATUS ChannelStatus;
			GetDevice().AutoCirculateGetStatus(Channel, ChannelStatus);
			return ChannelStatus;
		}

		bool ChannelThreadBase::CanInitialize() const
		{
			if (Channel == NTV2_CHANNEL_INVALID)
			{
				return false;
			}

			if (Helpers::GetNumberOfLinkChannel(GetOptions().TransportType) != 1)
			{
				if (Channel != Helpers::GetTransportTypeChannel(GetOptions().TransportType, Channel))
				{
					UE_LOGF(LogAjaCore, Error, "Device: Can't do a %s on channel '%d'. Did you mean channel '%d'?"
						, Helpers::TransportTypeToString(GetOptions().TransportType)
						, uint32_t(Channel) + 1
						, uint32_t(Helpers::GetTransportTypeChannel(GetOptions().TransportType, Channel)) + 1);
					return false;
				}
				if (UseKey())
				{
					if (KeyChannel != Helpers::GetTransportTypeChannel(GetOptions().TransportType, KeyChannel))
					{
						UE_LOGF(LogAjaCore, Error, "Device: Can't do a %s on channel '%d'. Did you mean channel '%d'?"
							, Helpers::TransportTypeToString(GetOptions().TransportType)
							, uint32_t(KeyChannel) + 1
							, uint32_t(Helpers::GetTransportTypeChannel(GetOptions().TransportType, KeyChannel)) + 1);
						return false;
					}
				}
				if (GetOptions().OutputReferenceType == AJA::EAJAReferenceType::EAJA_REFERENCETYPE_INPUT)
				{
					NTV2Channel SyncChannel = (NTV2Channel)(Options.SynchronizeChannelIndex - 1);
					if (SyncChannel != Helpers::GetTransportTypeChannel(GetOptions().TransportType, SyncChannel))
					{
						UE_LOGF(LogAjaCore, Error, "Device: Can't do a %s on channel '%d'. Did you mean channel '%d'?"
							, Helpers::TransportTypeToString(GetOptions().TransportType)
							, uint32_t(SyncChannel) + 1
							, uint32_t(Helpers::GetTransportTypeChannel(GetOptions().TransportType, SyncChannel)) + 1);
						return false;
					}
				}
			}

			return true;
		}

		bool ChannelThreadBase::DeviceThread_Initialize(DeviceConnection::CommandList& InCommandList)
		{
			bool bResult = ChannelThread_Initialization(InCommandList);
			Thread_OnInitializationCompleted(bResult);
			if (bResult)
			{
				FrameThread.Attach(&ChannelThreadBase::StaticThread, this);
				FrameThread.SetPriority(AJA_ThreadPriority_High);
				FrameThread.Start();
			}
			return bResult;
		}

		void ChannelThreadBase::Uninitialize()
		{
			AJAAutoLock AutoLock(&Lock);
			Options.CallbackInterface = nullptr;
			bStopRequested = true;
		}

		void ChannelThreadBase::DeviceThread_Destroy(DeviceConnection::CommandList& InCommandList)
		{
			delete TimecodeBurner;

			if (Device) // Device is valid when we are in the initialization thread
			{
				if (bConnectedKeyChannel)
				{
					InCommandList.SetChannelConnected(KeyChannel, false);
				}

				if (bRegisteredKeyChannel)
				{
					bool bAsOwner = true;
					InCommandList.UnregisterChannel(KeyChannel, bAsOwner);
				}

				if (bConnectedChannel)
				{
					InCommandList.SetChannelConnected(Channel, false);
				}

				//	Don't leave the audio system active after we exit
				if (NTV2_IS_VALID_AUDIO_SYSTEM(AudioSystem))
				{
					if (IsInput())
					{
						GetDevice().StopAudioInput(AudioSystem);
					}
					else
					{
						GetDevice().StopAudioOutput(AudioSystem);
					}
				}

				if (bRegisteredReference)
				{
					InCommandList.UnregisterReference(Channel);
				}

				if (bRegisteredChannel)
				{
					bool bAsOwner = true;
					InCommandList.UnregisterChannel(Channel, bAsOwner);
				}
			}

			//delete Card;

			Device.reset();
			//Card = nullptr;
			bRegisteredKeyChannel = false;
			bRegisteredReference = false;
			bRegisteredChannel = false;
			bConnectedChannel = false;
			bConnectedKeyChannel = false;
		}

		void ChannelThreadBase::Thread_OnInitializationCompleted(bool bSucceed)
		{
			AJAAutoLock AutoLock(&Lock);
			if (GetOptions().CallbackInterface)
			{
				GetOptions().CallbackInterface->OnInitializationCompleted(bSucceed);
			}
		}

		bool ChannelThreadBase::ChannelThread_Initialization(DeviceConnection::CommandList& InCommandList)
		{
			if (bStopRequested)
			{
				return false;
			}
			if (!CanInitialize())
			{
				return false;
			}

			assert(Device == nullptr);
			Device = DeviceCache::GetDevice(DeviceOption);
			if (Device == nullptr)
			{
				return false;
			}

			//assert(Card == nullptr);
			//Card = DeviceConnection::NewCNTV2Card(DeviceOption.DeviceIndex);
			//if (Card == nullptr)
			//{
			//	return false;
			//}

			if (bStopRequested)
			{
				return false;
			}

			if (!Helpers::TryVideoFormatIndexToNTV2VideoFormat(GetOptions().VideoFormatIndex, DesiredVideoFormat))
			{
				UE_LOGF(LogAjaCore, Error, "ConfigureVideo: The expected video format is invalid for %d.\n", uint32_t(Channel) + 1);
				return false;
			}

			if (!Helpers::ConvertTransportForDevice(Device->GetCard(), DeviceOption.DeviceIndex, Options.TransportType, DesiredVideoFormat))
			{
				return false;
			}

			if (Options.TransportType == ETransportType::TT_SdiSingle4kTSI && GetOptions().bUseKey)
			{
				UE_LOGF(LogAjaCore, Error, "ConfigureVideo: 4k Key and Fill output is not supported for this card.\n");
				return false;
			}

			if (Options.bDirectlyWriteAudio && Options.bUseAutoCirculating)
			{
				UE_LOGF(LogAjaCore, Error, "ConfigureVideo: Direct audio write is only available outside of auto-circulate mode..\n");
				return false;
			}
			
			const bool bConnectChannel = true;
			const bool bAsOwner = true;
			const bool bAsGenlock = false;

			// Get the override channel in case we need to reroute an input to a different channel.
			// ie. HDMI2 4K should go to channel 3
			Channel = Helpers::GetOverrideChannel(InputSource, Channel, Options.TransportType);
			Options.ChannelIndex = Channel + 1;

			if (IsInput() && Options.bAutoDetectFormat)
			{
				// We autodetect level B but we want level A
				DesiredVideoFormat = Helpers::GetLevelA(DesiredVideoFormat);
			}

			if (!InCommandList.RegisterChannel(GetOptions().TransportType, InputSource, Channel, IsInput(), bAsGenlock, bConnectChannel, GetOptions().TimecodeFormat, GetOptions().PixelFormat, DesiredVideoFormat, Options.HDROptions, bAsOwner, Options.bAutoDetectFormat))
			{
				return false;
			}

			bRegisteredChannel = true;

			BaseFrameIndex = Device->GetBaseFrameBufferIndex(Channel);

			if (bStopRequested)
			{
				return false;
			}

			if (!DeviceThread_Configure(InCommandList))
			{
				return false;
			}
			if (bStopRequested)
			{
				return false;
			}

			if (GetOptions().bBurnTimecode && UseTimecode())
			{
				TimecodeBurner = new AJATimeCodeBurn();
				TimecodeBurner->RenderTimeCodeFont(Helpers::ConvertToPixelFormat(GetOptions().PixelFormat), FormatDescriptor.numPixels, FormatDescriptor.numLines - FormatDescriptor.firstActiveLine);
			}

			if (GetOptions().bUseAutoCirculating)
			{
				if (!DeviceThread_ConfigureAutoCirculate(InCommandList))
				{
					return false;
				}
			}
			else
			{
				if (!DeviceThread_ConfigurePingPong(InCommandList))
				{
					return false;
				}
			}
			if (bStopRequested)
			{
				return false;
			}

			return !bStopRequested;
		}

		bool ChannelThreadBase::DeviceThread_Configure(DeviceConnection::CommandList& InCommandList)
		{
			if (!DeviceThread_ConfigureVideo(InCommandList))
			{
				return false;
			}
			if (bStopRequested)
			{
				return false;
			}
			if (!DeviceThread_ConfigureAnc(InCommandList))
			{
				return false;
			}
			if (bStopRequested)
			{
				return false;
			}
			if (!DeviceThread_ConfigureAudio(InCommandList))
			{
				return false;
			}
			if (bStopRequested)
			{
				return false;
			}
			return true;
		}

		bool ChannelThreadBase::DeviceThread_ConfigureVideo(DeviceConnection::CommandList& InCommandList)
		{
			VideoBufferSize = 0;
			VideoFormat = NTV2_FORMAT_UNKNOWN;

			bool bValidInput = UseVideo() && IsInput() && GetOptions().CallbackInterface != nullptr && Channel != NTV2_CHANNEL_INVALID;
			bool bValidOutput = UseVideo() && IsOutput() && GetOptions().CallbackInterface != nullptr && Channel != NTV2_CHANNEL_INVALID;
			bool bValidKey = !UseKey() || (KeyChannel != NTV2_CHANNEL_INVALID);

			if ((bValidInput || bValidOutput) && bValidKey)
			{
				if (IsInput())
				{
					std::string FailureReason;
					if (!Helpers::GetInputVideoFormat(Device->GetCard(), GetOptions().TransportType, Channel, InputSource, DesiredVideoFormat, VideoFormat, false, FailureReason))
					{
						UE_LOGF(LogAjaCore, Error, "Device: Initialization of the input failed for channel %d on device '%s'. '%s'", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str(), FailureReason.c_str());
						return false;
					}
				}
				else
				{
					assert(IsOutput());

					VideoFormat = DesiredVideoFormat;
					if (VideoFormat == NTV2_FORMAT_UNKNOWN)
					{
						UE_LOGF(LogAjaCore, Error, "ConfigureVideo: Unknown video format for output channel %d on device '%s'.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
						return false;
					}
				}

				if (!::NTV2DeviceCanDoVideoFormat(Device->GetCard()->GetDeviceID(), VideoFormat))
				{
					UE_LOGF(LogAjaCore, Error, "ConfigureVideo: The device '%s' doesn't support the video format '%s'.\n", GetDevice().GetDisplayName().c_str(), ::NTV2VideoFormatToString(VideoFormat).c_str());
					return false;
				}

				//	Set the frame buffer pixel format for all the channels on the device
				NTV2FrameBufferFormat PixelFormat = Helpers::ConvertPixelFormatToFrameBufferFormat(GetOptions().PixelFormat);

				if (!::NTV2DeviceCanDoFrameBufferFormat(Device->GetCard()->GetDeviceID(), PixelFormat))
				{
					UE_LOGF(LogAjaCore, Error, "ConfigureVideo: The device '%s' doesn't support the pixel format '%s'.\n", GetDevice().GetDisplayName().c_str(), ::NTV2FrameBufferFormatToString(PixelFormat).c_str());
					return false;
				}

				const int32_t NumberOfLinkChannel = Helpers::GetNumberOfLinkChannel(GetOptions().TransportType);

				const NTV2HDRXferChars ETOF = Helpers::ConvertToAjaHDRXferChars(GetOptions().HDROptions.EOTF);
				const NTV2HDRColorimetry Colorimetry = Helpers::ConvertToAjaHDRColorimetry(GetOptions().HDROptions.Gamut);
				const NTV2HDRLuminance Luminance = Helpers::ConvertToAjaHDRLuminance(GetOptions().HDROptions.Luminance);
				
				for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
				{
					AJA_CHECK(Device->GetCard()->SetFrameBufferFormat(NTV2Channel(int32_t(Channel) + ChannelIndex), PixelFormat, AJA_RETAIL_DEFAULT, ETOF, Colorimetry, Luminance));
				}

				// if output, set the Reference type
				if (IsOutput())
				{
					NTV2Channel SyncChannel = NTV2_CHANNEL1;
					if (GetOptions().OutputReferenceType == AJA::EAJAReferenceType::EAJA_REFERENCETYPE_INPUT)
					{
						if (GetOptions().SynchronizeChannelIndex > NTV2_MAX_NUM_CHANNELS || GetOptions().SynchronizeChannelIndex < 1)
						{
							UE_LOGF(LogAjaCore, Error, "ConfigureVideo: The synchronization port index '%d' is invalid.\n", GetOptions().SynchronizeChannelIndex);
							return false;
						}
						else
						{
							SyncChannel = (NTV2Channel)(Options.SynchronizeChannelIndex - 1);
						}
					}

					if (!InCommandList.RegisterReference(GetOptions().OutputReferenceType, SyncChannel))
					{
						return false;
					}

					bRegisteredReference = true;
				}


				NTV2VANCMode VancMode(NTV2_VANCMODE_INVALID);
				GetDevice().GetVANCMode(VancMode);
				NTV2Standard Standard = ::GetNTV2StandardFromVideoFormat(VideoFormat);

				VideoBufferSize = ::GetVideoWriteSize(VideoFormat, PixelFormat, VancMode);
				FormatDescriptor = NTV2FormatDescriptor(Standard, PixelFormat);

				if (!ChannelThread_ConfigureRouting(InCommandList, PixelFormat))
				{
					return false;
				}
			}
			else
			{
				UE_LOGF(LogAjaCore, Error, "ConfigureVideo: Invalid video options for channel %d on device '%s'.", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
				return false;
			}

			return true;
		}


		bool ChannelThreadBase::ChannelThread_ConfigureRouting(DeviceConnection::CommandList& InCommandList, NTV2FrameBufferFormat InPixelFormat)
		{
			if (IsInput())
			{
				if (!InCommandList.IsChannelConnect(Channel))
				{
					if (Helpers::IsSdiTransport(GetOptions().TransportType))
					{
						const bool bIsSignalRgb = false;
						const bool bWillUseKey = false;
						Helpers::RouteSdiSignal(GetDevicePtr(), GetOptions().TransportType, Channel, VideoFormat, InPixelFormat, IsInput(), bIsSignalRgb, bWillUseKey);
					}
					else if (Helpers::IsHdmiTransport(GetOptions().TransportType))
					{
						constexpr bool bIsInput = true;
						Helpers::RouteHdmiSignal(GetDevicePtr(), GetOptions().TransportType, InputSource, Channel, InPixelFormat, bIsInput);
					}
					InCommandList.SetChannelConnected(Channel, true);
					bConnectedChannel = true;
				}
			}
			else
			{
				AJA_CHECK(IsOutput());

				if (Helpers::IsSdiTransport(GetOptions().TransportType))
				{
					if (!Helpers::SetSDIOutLevelAtoLevelBConversion(Device->GetCard(), GetOptions().TransportType, Channel, VideoFormat, GetOptions().bConvertOutputLevelAToB))
					{
						if (GetOptions().bConvertOutputLevelAToB)
						{
							UE_LOGF(LogAjaCore, Warning,  "ConfigureVideo: Can't convert level A to B for channel %d on device '%s'.", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
						}
					}

					if (!InCommandList.IsChannelConnect(Channel))
					{
						const bool bIsSignalRgb = false;
						Helpers::RouteSdiSignal(GetDevicePtr(), GetOptions().TransportType, Channel, VideoFormat, InPixelFormat, IsInput(), bIsSignalRgb, UseKey());

						InCommandList.SetChannelConnected(Channel, true);
						bConnectedChannel = true;
					}

					const NTV2Standard OutputStandard = ::GetNTV2StandardFromVideoFormat(VideoFormat);
					AJA_CHECK(GetDevice().SetSDIOutputStandard(Channel, OutputStandard));

					if (UseKey())
					{
						bConnectedChannel = false;
						constexpr bool bAsOwner = true;
						constexpr bool bAsGenlock = false;

						if (!InCommandList.RegisterChannel(GetOptions().TransportType, KeySource, KeyChannel, IsInput(), bAsGenlock, bConnectedChannel, ETimecodeFormat::TCF_None, GetOptions().PixelFormat, DesiredVideoFormat, Options.HDROptions, bAsOwner, Options.bAutoDetectFormat))
						{
							return false;
						}
						bRegisteredKeyChannel = true;

						if (!InCommandList.IsChannelConnect(KeyChannel))
						{
							if (!Helpers::SetSDIOutLevelAtoLevelBConversion(Device->GetCard(), GetOptions().TransportType, KeyChannel, VideoFormat, GetOptions().bConvertOutputLevelAToB))
							{
								if (GetOptions().bConvertOutputLevelAToB)
								{
									UE_LOGF(LogAjaCore, Warning,  "ConfigureVideo: Can't convert level A to B on key for channel %d on device '%s'.", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
								}
							}
							Helpers::RouteKeySignal(Device->GetCard(), GetOptions().TransportType, Channel, KeyChannel, InPixelFormat, false);

							InCommandList.SetChannelConnected(KeyChannel, true);
							bConnectedKeyChannel = true;
						}
					}
				}
				else
				{
					if (!InCommandList.IsChannelConnect(Channel))
					{
						const bool bIsSignalRgb = false;
						constexpr bool bIsInput = true;
						Helpers::RouteHdmiSignal(GetDevicePtr(), GetOptions().TransportType, InputSource, Channel, InPixelFormat, bIsInput);

						InCommandList.SetChannelConnected(Channel, true);
						bConnectedChannel = true;
					}

					const NTV2Standard OutputStandard = ::GetNTV2StandardFromVideoFormat(VideoFormat);
					AJA_CHECK(GetDevice().SetHDMIOutVideoStandard(OutputStandard));

					if (UseKey())
					{
						const bool bUseKeyConnectedChannel = false;
						constexpr bool bAsOwner = true;
						constexpr bool bAsGenlock = false;

						if (!InCommandList.RegisterChannel(GetOptions().TransportType, KeySource, KeyChannel, IsInput(), bAsGenlock, bUseKeyConnectedChannel, ETimecodeFormat::TCF_None, GetOptions().PixelFormat, DesiredVideoFormat, Options.HDROptions, bAsOwner, Options.bAutoDetectFormat))
						{
							return false;
						}
						bRegisteredKeyChannel = true;

						if (!InCommandList.IsChannelConnect(KeyChannel))
						{
							Helpers::RouteKeySignal(Device->GetCard(), GetOptions().TransportType, Channel, KeyChannel, InPixelFormat, false);

							InCommandList.SetChannelConnected(KeyChannel, true);
							bConnectedKeyChannel = true;
						}
					}
				}
			}

			return true;
		}

		bool ChannelThreadBase::DeviceThread_ConfigureAnc(DeviceConnection::CommandList& InCommandList)
		{
			AncBufferSize = 0;
			AncF2BufferSize = 0;

			//	Make sure the device actually supports custom anc before using it...
			if (UseAncillary() || UseAncillaryField2())
			{
				bool bSupportAnc = ::NTV2DeviceCanDoCustomAnc(GetDevice().GetDeviceID());
				if (!bSupportAnc)
				{
					UE_LOGF(LogAjaCore, Warning,  "ConfigureAncillary: Device doesn't support Anc.\n");
					Options.bUseAncillary = false;
				}
			}

			const uint32_t NTV2_ANCSIZE_MAX = 0x2000;
			if (UseAncillary())
			{
				AncBufferSize = NTV2_ANCSIZE_MAX;
			}
			if (UseAncillaryField2())
			{
				AncF2BufferSize = NTV2_ANCSIZE_MAX;
			}

			return true;
		}

		bool ChannelThreadBase::DeviceThread_ConfigureAudio(DeviceConnection::CommandList& InCommandList)
		{
			AudioBufferSize = 0;

			if (UseAudio() && GetOptions().CallbackInterface != nullptr)
			{
				AudioSystem = NTV2InputSourceToAudioSystem(InputSource);
				if (NTV2_IS_VALID_AUDIO_SYSTEM(AudioSystem) == false)
				{
					UE_LOGF(LogAjaCore, Error, "ConfigureAudio: Could not get valid AudioSystem from InputSource %d for channel %d on device %s.", uint32_t(InputSource), uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
					return false;
				}
			}
			else
			{
				if (UseAudio())
				{
					UE_LOGF(LogAjaCore, Error, "ConfigureAudio: Invalid audio options for channel %d on device %s.", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
					return false;
				}
			}

			if (NTV2_IS_VALID_AUDIO_SYSTEM(AudioSystem))
			{
				CNTV2Card& Card = GetDevice();

				if (IsInput())
				{
					//	Have the audio system capture audio from the designated device input (i.e., ch1 uses SDIIn1, ch2 uses SDIIn2, etc.)...
					Card.SetAudioSystemInputSource(AudioSystem, NTV2_AUDIO_EMBEDDED, ::NTV2ChannelToEmbeddedAudioInput(Channel));
				}

				ULWord MaxNumOfChannels = ::NTV2DeviceGetMaxAudioChannels(Card.GetDeviceID());
				ULWord NumberOfAudioChannel = GetOptions().NumberOfAudioChannel;

				//	If there are 4096 pixels on a line instead of 3840, reduce the number of audio channels
				//	This is because HANC is narrower, and has space for only 8 channels
				if (NTV2_IS_4K_4096_VIDEO_FORMAT(VideoFormat) && (MaxNumOfChannels > 8))
				{
					MaxNumOfChannels = 8;
				}

				if (NumberOfAudioChannel > MaxNumOfChannels)
				{
					NumberOfAudioChannel = MaxNumOfChannels;
					UE_LOGF(LogAjaCore, Warning,  "ConfigureAudio: Changed number of audio channel to %d.\n", NumberOfAudioChannel);
				}

				Card.SetNumberAudioChannels(NumberOfAudioChannel, AudioSystem);

				//	The on-device audio buffer should be 4MB to work best across all devices & platforms...
				Card.SetAudioBufferSize(NTV2_AUDIO_BUFFER_BIG, AudioSystem);

				if (IsOutput())
				{
					//	Set up the output audio embedders...
					if (::NTV2DeviceGetNumAudioSystems(Card.GetDeviceID()) > 1)
					{
						Card.SetSDIOutputAudioSystem(Channel, AudioSystem);
					}
				}

				Card.SetAudioLoopBack(NTV2_AUDIO_LOOPBACK_OFF, AudioSystem);

				const uint32_t NTV2_AUDIOSIZE_MAX = (4 * 1024 * 1024);
				AudioBufferSize = NTV2_AUDIOSIZE_MAX;
			}

			return true;
		}

		void ChannelThreadBase::StaticThread(AJAThread* pThread, void* pContext)
		{
			ChannelThreadBase* Channel = reinterpret_cast<ChannelThreadBase*>(pContext);
			if (Channel->GetOptions().bUseAutoCirculating)
			{
				Channel->Thread_AutoCirculateLoop();
			}
			else
			{
				Channel->Thread_PingPongLoop();
			}
		}

		void ChannelThreadBase::BurnTimecode(const AJA::FTimecode& InTimecode, uint8_t* InVideoBuffer)
		{
			if (TimecodeBurner)
			{
				const TimecodeFormat TcFormat = Helpers::ConvertToTimecodeFormat(VideoFormat);
				CRP188 FrameRP188Info(InTimecode.Frames, InTimecode.Seconds, InTimecode.Minutes, InTimecode.Hours, TcFormat);

				std::string TimeCodeString;
				FrameRP188Info.GetRP188Str(TimeCodeString);

				char* pVideoBuffer = reinterpret_cast<char*>(InVideoBuffer + FormatDescriptor.firstActiveLine * FormatDescriptor.linePitch * 4);
				TimecodeBurner->BurnTimeCode(pVideoBuffer, TimeCodeString.c_str(), GetOptions().BurnTimecodePercentY);
			}
		}

		void ChannelThreadBase::BurnTimecode(const AJA::FTimecode& InTimecode, const AJA::FTimecode& InTimecode2, uint8_t* InVideoBuffer)
		{
			if (TimecodeBurner)
			{
				const TimecodeFormat TcFormat = Helpers::ConvertToTimecodeFormat(VideoFormat);
				CRP188 FrameRP188Info(InTimecode.Frames, InTimecode.Seconds, InTimecode.Minutes, InTimecode.Hours, TcFormat);

				CRP188 FrameRP188Info2(InTimecode2.Frames, InTimecode2.Seconds, InTimecode2.Minutes, InTimecode2.Hours, TcFormat);

				std::string TimeCodeString;
				FrameRP188Info.GetRP188Str(TimeCodeString);
				std::string TimeCodeString2;
				FrameRP188Info2.GetRP188Str(TimeCodeString2);

				char* pVideoBuffer = reinterpret_cast<char*>(InVideoBuffer + FormatDescriptor.firstActiveLine * FormatDescriptor.linePitch * 4);
				TimecodeBurner->BurnTimeCode(pVideoBuffer, TimeCodeString, GetOptions().BurnTimecodePercentY);

				TimecodeBurner->BurnTimeCode(pVideoBuffer, TimeCodeString2, 20);
			}
		}
	}
}