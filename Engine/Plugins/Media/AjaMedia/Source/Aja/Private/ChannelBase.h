// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers.h"

#include "Device.h"

#include <memory>

namespace AJA
{
	namespace Private
	{
		struct IOChannelInitialize_DeviceCommand;
		struct IOChannelUninitialize_DeviceCommand;
		class ChannelThreadBase;

		/* IOChannelInitialize_DeviceCommand definition
		*****************************************************************************/
		struct IOChannelInitialize_DeviceCommand : DeviceCommand
		{
			IOChannelInitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<ChannelThreadBase>& InChannel);
			
			virtual void Execute(DeviceConnection::CommandList& InCommandList) override;

			std::weak_ptr<ChannelThreadBase> ChannelThread;
		};

		/* IOChannelUninitialize_DeviceCommand definition
		*****************************************************************************/
		struct IOChannelUninitialize_DeviceCommand : DeviceCommand
		{
			IOChannelUninitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<ChannelThreadBase>& InChannel, TPromise<void> InCompletionPromise = {});
			
			virtual void Execute(DeviceConnection::CommandList& InCommandList) override;

			std::shared_ptr<ChannelThreadBase> ChannelThread;

		private:
			/** Completion promise fired when the uninitialized command is done. */
			TPromise<void> CompletionPromise;
		};

		/* ChannelThreadBase OutputChannelThread
		*****************************************************************************/
		class ChannelThreadBase
		{
			friend IOChannelInitialize_DeviceCommand;
			friend IOChannelUninitialize_DeviceCommand;

		public:
			ChannelThreadBase() = delete;
			ChannelThreadBase(const ChannelThreadBase&) = delete;
			ChannelThreadBase& operator=(const ChannelThreadBase&) = delete;

			ChannelThreadBase(const AJADeviceOptions& InDevice, const AJAInputOutputChannelOptions& InOptions);
			virtual ~ChannelThreadBase();

			virtual bool CanInitialize() const;
			bool DeviceThread_Initialize(DeviceConnection::CommandList& InCommandList);
			virtual void Uninitialize();
			virtual void DeviceThread_Destroy(DeviceConnection::CommandList& InCommandList);

		public:
			const AJAInputOutputChannelOptions& GetOptions() const { return Options; }
			//CNTV2Card& GetDevice() { assert(Card); return *Card; }
			//CNTV2Card& GetDevice() const { assert(Card); return *Card; }
			CNTV2Card& GetDevice() const { return *Device->GetCard(); }
			CNTV2Card* GetDevicePtr() const { return Device->GetCard(); }

			AUTOCIRCULATE_STATUS GetCurrentAutoCirculateStatus() const;

			bool IsInput() const { return !IsOutput(); }
			bool IsOutput() const { return Options.bOutput; }

			bool UseAudio() const { return Options.bUseAudio; }
			bool UseVideo() const { return Options.bUseVideo; }
			bool UseAncillary() const { return Options.bUseAncillary; }
			bool UseAncillaryField2() const { return UseAncillary() && !::IsProgressiveTransport(DesiredVideoFormat); }

			bool UseKey() const { return Options.bUseKey; }
			bool UseTimecode() const { return Options.TimecodeFormat != ETimecodeFormat::TCF_None; }

		public:
			const AJADeviceOptions DeviceOption;
			std::shared_ptr<DeviceConnection> Device;
			//CNTV2Card* Card;

			AJALock Lock; // for callback

			NTV2Channel Channel;
			NTV2Channel KeyChannel;

			NTV2InputSource InputSource;
			NTV2InputSource KeySource;
			NTV2AudioSystem AudioSystem;

			NTV2FormatDescriptor FormatDescriptor;
			// DesiredVideoFormat and VideoFormat should match. There was a reason to have the 2 before (psf and levelB) but now they should be the same.
			NTV2VideoFormat DesiredVideoFormat;
			NTV2VideoFormat VideoFormat;

			AJATimeCodeBurn* TimecodeBurner;

			uint32_t BaseFrameIndex;

			uint32_t AncBufferSize;
			uint32_t AncF2BufferSize;
			uint32_t AudioBufferSize;
			uint32_t VideoBufferSize;

			bool bRegisteredChannel;
			bool bRegisteredKeyChannel;
			bool bRegisteredReference;
			bool bConnectedChannel;
			bool bConnectedKeyChannel;

		protected:
			virtual bool ChannelThread_Initialization(DeviceConnection::CommandList& InCommandList);
			virtual bool DeviceThread_Configure(DeviceConnection::CommandList& InCommandList);
			virtual bool DeviceThread_ConfigureAnc(DeviceConnection::CommandList& InCommandList);
			virtual bool DeviceThread_ConfigureAudio(DeviceConnection::CommandList& InCommandList);
			virtual bool DeviceThread_ConfigureVideo(DeviceConnection::CommandList& InCommandList);

			virtual bool DeviceThread_ConfigureAutoCirculate(DeviceConnection::CommandList& InCommandList) = 0;
			virtual bool DeviceThread_ConfigurePingPong(DeviceConnection::CommandList& InCommandList) = 0;

			virtual void Thread_AutoCirculateLoop() = 0;
			virtual void Thread_PingPongLoop() = 0;

			void Thread_OnInitializationCompleted(bool bSucceed);

			// Encode timecode onto a video buffer.
			void BurnTimecode(const AJA::FTimecode& InTimecode, uint8_t* InVideoBuffer);
			// Burn two timecodes on the frame (Used for interlace debugging to identify the timecode of the two frames that make up an output frame.)
			void BurnTimecode(const AJA::FTimecode& InTimecode, const AJA::FTimecode& InTimecode2, uint8_t* InVideoBuffer);

		private:
			static void StaticThread(AJAThread* pThread, void* pContext);
			bool ChannelThread_ConfigureRouting(DeviceConnection::CommandList& InCommandList, NTV2FrameBufferFormat PixelFormat);

		protected:
			volatile bool bStopRequested;
			AJAInputOutputChannelOptions Options;
			AJAThread FrameThread;
		};
	}
}
