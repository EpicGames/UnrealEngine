// Copyright Epic Games, Inc. All Rights Reserved.

#include "Device.h"

#include <algorithm>
#include <chrono>

namespace AJA
{
	namespace Private
	{
		static ULWord UEAppType = AJA_FOURCC('U', 'E', '5', '.');
		static DeviceCache Cache;

		/* DeviceCache implementation
		*****************************************************************************/

		std::shared_ptr<DeviceConnection> DeviceCache::GetDevice(const AJADeviceOptions& InDeviceOption)
		{
			if (InDeviceOption.DeviceIndex >= MaxNumberOfDevice || InDeviceOption.DeviceIndex < 0)
			{
				UE_LOGF(LogAjaCore, Error, "DeviceCache: The device index '%d' is invalid.\n", InDeviceOption.DeviceIndex);
				return std::shared_ptr<DeviceConnection>();
			}

			{
				AJAAutoLock AutoLock(&Cache.Lock);

				auto Result = Cache.DeviceInstance[InDeviceOption.DeviceIndex].lock();
				if (!Result)
				{
					Result.reset(new DeviceConnection(InDeviceOption));
					Cache.DeviceInstance[InDeviceOption.DeviceIndex] = Result;
				}

				return Result;
			}
		}

		void DeviceCache::RemoveDevice(const std::shared_ptr<DeviceConnection>& InDevice)
		{
			if (InDevice)
			{
				Cache.DeviceInstance[InDevice->DeviceOption.DeviceIndex].reset();
			}
		}

		/* SyncThreadInfoThread implementation
		*****************************************************************************/
		DeviceConnection::SyncThreadInfoThread::SyncThreadInfoThread(DeviceConnection* InDeviceConnection, ChannelInfo* InChannelInfo)
			: ChannelInfoPtr(InChannelInfo)
			, DeviceConnectionPtr(InDeviceConnection)
		{
		}

		bool DeviceConnection::SyncThreadInfoThread::WaitForVerticalInterrupt(DeviceConnection* InDeviceConnection, ChannelInfo* InChannelInfo, uint32& OutSyncCount)
		{
			assert(InDeviceConnection);
			assert(InChannelInfo);

			bool bResult = true;

			if (InChannelInfo->RefCounter > 0)
			{
				ULWord BeforeCount, AfterCount = 0;
				bool bFirstCountValid, bSecondCountValid = false;
				if (InChannelInfo->bIsInput)
				{
					do
					{
						bFirstCountValid = InDeviceConnection->GetCard()->GetInputVerticalInterruptCount(BeforeCount, InChannelInfo->Channel);
						bResult = InDeviceConnection->GetCard()->WaitForInputVerticalInterrupt(InChannelInfo->Channel);
						bSecondCountValid = InDeviceConnection->GetCard()->GetInputVerticalInterruptCount(AfterCount, InChannelInfo->Channel);
					} while (bFirstCountValid && bSecondCountValid && BeforeCount == AfterCount && bResult);
				}
				else
				{
					do
					{
						bFirstCountValid = InDeviceConnection->GetCard()->GetOutputVerticalInterruptCount(BeforeCount, InChannelInfo->Channel);
						bResult = InDeviceConnection->GetCard()->WaitForOutputVerticalInterrupt(InChannelInfo->Channel);
						bSecondCountValid = InDeviceConnection->GetCard()->GetOutputVerticalInterruptCount(AfterCount, InChannelInfo->Channel);
					} while (bFirstCountValid && bSecondCountValid && BeforeCount == AfterCount && bResult);
				}
				OutSyncCount = AfterCount;
			}

			return bResult;
		}

		bool DeviceConnection::SyncThreadInfoThread::IsCurrentField(DeviceConnection* InDeviceConnection, ChannelInfo* InChannelInfo, NTV2FieldID InFieldId)
		{
			assert(InDeviceConnection);
			assert(InChannelInfo);

			bool bResult = false;

			if (InChannelInfo->RefCounter > 0)
			{
				if (InChannelInfo->bIsInput)
				{
					bResult = Helpers::IsCurrentInputField(InDeviceConnection->GetCard(), InChannelInfo->Channel, InFieldId);
				}
				else
				{
					bResult = Helpers::IsCurrentOutputField(InDeviceConnection->GetCard(), InChannelInfo->Channel, InFieldId);
				}
			}

			return bResult;
		}

		bool DeviceConnection::SyncThreadInfoThread::Wait_ExternalThread()
		{
			using namespace std::chrono_literals;
			std::unique_lock<std::mutex> LockGuard(SyncMutex);
			bool bResult = SyncCondition.wait_for(LockGuard, 50ms) == std::cv_status::no_timeout && bWaitResult;
			return bResult;
		}

		bool DeviceConnection::SyncThreadInfoThread::ThreadLoop()
		{
			// do the wait
			uint32 NewSyncCount = 0;
			bWaitResult = WaitForVerticalInterrupt(DeviceConnectionPtr, ChannelInfoPtr, NewSyncCount);
			SyncCondition.notify_all();

			OnVerticalInterruptDelegate.Broadcast(NewSyncCount);

			return true;
		}

		/* WaitForInputInfo implementation
		*****************************************************************************/
		bool DeviceConnection::WaitForInputInfo::WaitForInput()
		{
			using namespace std::chrono_literals;
			std::unique_lock<std::mutex> LockGuard(SyncMutex);
			return SyncCondition.wait_for(LockGuard, 50ms) == std::cv_status::no_timeout;
		}

		void DeviceConnection::WaitForInputInfo::NotifyAll()
		{
			SyncCondition.notify_all();
		}

		/* ChannelInfo implementation
		*****************************************************************************/
		DeviceConnection::ChannelInfo::ChannelInfo()
			: TransportType(ETransportType::TT_SdiSingle)
			, Channel(NTV2_CHANNEL_INVALID)
			, VideoFormat(NTV2_FORMAT_UNKNOWN)
			, TimecodeFormat(ETimecodeFormat::TCF_None)
			, RefCounter(0)
			, BaseFrameBufferIndex(InvalidFrameBufferIndex)
			, bConnected(false)
			, bIsOwned(false)
			, bIsInput(false)
			, bIsAutoDetected(false)
			, bGenlockChannel(false)
		{
		}

		/* CommandList implementation
		*****************************************************************************/
		DeviceConnection::CommandList::CommandList(DeviceConnection& InConnection)
			: Connection(InConnection)
		{}

		bool DeviceConnection::CommandList::RegisterChannel(ETransportType InTransportType, NTV2InputSource InInputSource, NTV2Channel InChannel, bool bInAsInput, bool bInAsGenlock, bool bConnectChannel, ETimecodeFormat InTimecodeFormat, EPixelFormat InUEPixelFormat, NTV2VideoFormat InDesiredInputFormat, bool bInAsOwner, bool bInIsAutoDetected)
		{
			FAjaHDROptions UnusedOptions;
			return RegisterChannel(InTransportType, InInputSource, InChannel, bInAsInput, bInAsGenlock, bConnectChannel, InTimecodeFormat, InUEPixelFormat, InDesiredInputFormat, UnusedOptions, bInAsOwner, bInIsAutoDetected);
		}

		bool DeviceConnection::CommandList::RegisterChannel(ETransportType InTransportType, NTV2InputSource InInputSource, NTV2Channel InChannel, bool bInIsInput, bool bInAsGenlock, bool bConnectChannel, ETimecodeFormat InTimecodeFormat, EPixelFormat InUEPixelFormat, NTV2VideoFormat InDesiredInputFormat, FAjaHDROptions& InOutHDROptions, bool bInAsOwner, bool bInIsAutoDetected)
		{
			AJAAutoLock Lock(&Connection.ChannelLock);

			std::vector<ChannelInfo*>& ConnectionChannelInfos = Connection.ChannelInfos;

			if (!Connection.Lock_IsInitialize() && !Connection.Lock_Initialize())
			{
				return false;
			}

			// Find if we have already register that channel
			bool bResult = true;
			NTV2VideoFormat VideoFormat = NTV2_FORMAT_UNKNOWN;
			auto FoundIterator = std::find_if(std::begin(ConnectionChannelInfos), std::end(ConnectionChannelInfos), [=](const ChannelInfo* Other) { return Other->Channel == InChannel; });
			bool bReinitialize = FoundIterator != std::end(ConnectionChannelInfos) && (*FoundIterator)->bIsAutoDetected;
			
			if (FoundIterator == std::end(ConnectionChannelInfos))
			{
				bResult = Connection.Lock_EnableChannel_SDILinkHelper(Connection.Card, InTransportType, InChannel);

				if (bResult)
				{
					bResult = Connection.Lock_EnableChannel_Helper(Connection.Card, InTransportType, InInputSource, InChannel, bInIsInput, bConnectChannel, bInIsAutoDetected, InTimecodeFormat, InUEPixelFormat, InDesiredInputFormat, VideoFormat, InOutHDROptions);
				}

				uint32_t NewBaseFrameBufferIndex = InvalidFrameBufferIndex;
				if (bResult)
				{
					NewBaseFrameBufferIndex = Connection.Lock_AcquireBaseFrameIndex(InChannel, VideoFormat);
					bResult = NewBaseFrameBufferIndex != InvalidFrameBufferIndex;
				}

				if (bResult)
				{
					Connection.Lock_SubscribeChannel_Helper(Connection.Card, InChannel, bInIsInput);

					if (bInIsInput)
					{
						AJA_CHECK(Connection.Card->SetInputFrame(InChannel, NewBaseFrameBufferIndex));
					}
					else
					{
						AJA_CHECK(Connection.Card->SetOutputFrame(InChannel, NewBaseFrameBufferIndex));
					}

					ChannelInfo* Info = new ChannelInfo();
					Info->TransportType = InTransportType;
					Info->Channel = InChannel;
					Info->VideoFormat = VideoFormat;
					Info->TimecodeFormat = InTimecodeFormat;
					Info->RefCounter = 1;
					Info->BaseFrameBufferIndex = NewBaseFrameBufferIndex;
					Info->bIsOwned = bInAsOwner;
					Info->bIsInput = bInIsInput;
					Info->bConnected = bConnectChannel;
					Info->bIsAutoDetected = bInIsAutoDetected;
					Info->bGenlockChannel = bInAsGenlock;

					// Spawn a sync thread if there is more than 1 connection that can potentially wait on the same event
					Info->SyncThread.reset(new SyncThreadInfoThread(&Connection, Info));
					Info->SyncThread->SetPriority(AJA_ThreadPriority_High);
					Info->SyncThread->Start();

					{
						AJAAutoLock AutoLock(&Connection.ChannelInfoLock);
						ConnectionChannelInfos.push_back(Info);
					}
				}
			}
			else
			{
				ChannelInfo* FoundChannel = *FoundIterator;

				// If channel was used for Genlock, reinitialize channel since routing wasn't done.
				if (bReinitialize || (FoundChannel->bGenlockChannel && !bInAsGenlock))
				{
					if (!FoundChannel->bIsAutoDetected && !(FoundChannel->bGenlockChannel && !bInAsGenlock))
					{
						UE_LOGF(LogAjaCore, Warning,  "Device: Autodetect reinitialized channel '%d' on device '%s' but the channel was not in autodetect mode.\n"
							, uint32_t(InChannel) + 1, Connection.Card->GetDisplayName().c_str());
					}

					bResult = Connection.Lock_EnableChannel_SDILinkHelper(Connection.Card, InTransportType, InChannel);

					if (bResult)
					{
						bResult = Connection.Lock_EnableChannel_Helper(Connection.Card, InTransportType, InInputSource, InChannel, bInIsInput, bConnectChannel, bInIsAutoDetected, InTimecodeFormat, InUEPixelFormat, InDesiredInputFormat, VideoFormat, InOutHDROptions);
					}
				}

				if (bResult && FoundChannel->bIsOwned && bInAsOwner)
				{
					UE_LOGF(LogAjaCore, Error, "Device: Trying to enable the channel '%d' on device '%s' as the owner but it was already enabled. Do you already have an '%s' running on the same channel?\n"
						, uint32_t(InChannel) + 1, Connection.Card->GetDisplayName().c_str()
						, FoundChannel->bIsInput ? "input" : "output");
					bResult = false;
				}

				if (bResult && FoundChannel->TransportType != InTransportType)
				{
					UE_LOGF(LogAjaCore, Error, "Device: Trying to enable the channel '%d' on device '%s' as a '%s' but it was already enabled as a '%s'.\n"
						, uint32_t(InChannel) + 1
						, Connection.Card->GetDisplayName().c_str()
						, Helpers::TransportTypeToString(InTransportType)
						, Helpers::TransportTypeToString(FoundChannel->TransportType));
					bResult = false;
				}

				if (bResult && FoundChannel->bIsInput != bInIsInput)
				{
					UE_LOGF(LogAjaCore, Error, "Device: Trying to enable the channel '%d' on device '%s' as an '%s' but it was already enabled as an '%s'.\n"
						, uint32_t(InChannel) + 1
						, Connection.Card->GetDisplayName().c_str()
						, bInIsInput ? "input" : "output"
						, FoundChannel->bIsInput ? "input" : "output");
					bResult = false;
				}

				if (bResult)
				{
					std::string FailureReason;
					if (bInAsGenlock || FoundChannel->bGenlockChannel)
					{
						VideoFormat = InDesiredInputFormat;
					} 
					else if (!Helpers::GetInputVideoFormat(Connection.Card, InTransportType, InChannel, InInputSource, InDesiredInputFormat, VideoFormat, false, FailureReason))
					{
						UE_LOGF(LogAjaCore, Error, "Device: Initialization of the input failed for channel %d on device %s. %s", uint32_t(InChannel) + 1, Connection.Card->GetDisplayName().c_str(), FailureReason.c_str());
						bResult = false;
					}

					if (bResult && !bInIsAutoDetected && VideoFormat != FoundChannel->VideoFormat)
					{
						const bool bForRetailDisplay = true;
						UE_LOGF(LogAjaCore, Error, "Device: Try to enable with the video format '%s' the channel '%d' on device '%s' that was already enabled with '%s'.\n"
							, NTV2FrameRateToString(GetNTV2FrameRateFromVideoFormat(VideoFormat), bForRetailDisplay).c_str()
							, uint32_t(InChannel) + 1, Connection.Card->GetDisplayName().c_str()
							, NTV2FrameRateToString(GetNTV2FrameRateFromVideoFormat(FoundChannel->VideoFormat), bForRetailDisplay).c_str());
						bResult = false;
					}
				}

				if (bResult && FoundChannel->TimecodeFormat != InTimecodeFormat && !bInIsAutoDetected && bInAsGenlock)
				{
					UE_LOGF(LogAjaCore, Warning,  "Device: Try to enable the channel '%d' on device '%s' with a timecode format that is not the same as the previous one. Timecode may not be decoded properly.\n"
						, uint32_t(InChannel) + 1, Connection.Card->GetDisplayName().c_str());
				}

				if (bResult)
				{
					++FoundChannel->RefCounter;

					if (bInAsOwner)
					{
						FoundChannel->bIsOwned = bInAsOwner;
					}
				}
			}

			//Verify if Channel1 is used. If not, we need to be sure the card's main format is configured
			if (bResult && !::NTV2DeviceCanDoMultiFormat(Connection.Card->GetDeviceID()))
			{
				auto Found = std::find_if(std::begin(Connection.ChannelInfos), std::end(Connection.ChannelInfos), [=](const ChannelInfo* Info) { return Info->Channel == NTV2_CHANNEL1; });
				if (Found == std::end(Connection.ChannelInfos))
				{
					//Nothing set on channel 1, set it up so the card is correctly initialized
					AJA_CHECK(Connection.Card->SetVideoFormat(VideoFormat, AJA_RETAIL_DEFAULT, false, NTV2_CHANNEL1));
				}
			}

			if (!bResult && !Connection.Lock_IsInitialize())
			{
				Connection.Lock_UnInitialize();
			}

			return bResult;
		}

		bool DeviceConnection::CommandList::UnregisterChannel(NTV2Channel InChannel, bool bInAsOwner)
		{
			AJAAutoLock Lock(&Connection.ChannelLock);

			assert(Connection.Lock_IsInitialize());

			std::vector<ChannelInfo*>& ConnectionChannelInfos = Connection.ChannelInfos;

			// Find if we have already register that channel
			auto FoundIterator = std::find_if(std::begin(ConnectionChannelInfos), std::end(ConnectionChannelInfos), [=](const ChannelInfo* Other) { return Other->Channel == InChannel; });
			if (FoundIterator == std::end(ConnectionChannelInfos))
			{
				UE_LOGF(LogAjaCore, Error, "Device: Try to unregister the channel '%d' on device '%s' when it's not registered.\n", uint32_t(InChannel) + 1, Connection.Card->GetDisplayName().c_str());
				assert(false);
				return false;
			}

			ChannelInfo* FoundChannel = *FoundIterator;
			--FoundChannel->RefCounter;
			if (FoundChannel->RefCounter == 0)
			{
				// Stop the sync thread
				if (FoundChannel->SyncThread)
				{
					FoundChannel->SyncThread->Stop(50);
					FoundChannel->SyncThread.release();
				}

				if (Helpers::IsTsiRouting(FoundChannel->TransportType))
				{
					AJA_CHECK(Connection.Card->SetTsiFrameEnable(false, InChannel));
				}
				else if (FoundChannel->TransportType == ETransportType::TT_SdiDual)
				{
					AJA_CHECK(Connection.Card->SetSmpte372(false, InChannel));
				}
				else if (FoundChannel->TransportType == ETransportType::TT_SdiQuadSQ)
				{
					AJA_CHECK(Connection.Card->Set4kSquaresEnable(false, InChannel));
				}

				// Do not disable the interrupt. The ControlRoom need the interrupt enabled.
				if (FoundChannel->bIsInput)
				{
					AJA_CHECK(Connection.Card->UnsubscribeInputVerticalEvent(InChannel));
				}
				else
				{
					AJA_CHECK(Connection.Card->UnsubscribeOutputVerticalEvent(InChannel));
				}
				const int32_t NumberOfLinkChannel = Helpers::GetNumberOfLinkChannel(FoundChannel->TransportType);
				for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
				{
					AJA_CHECK(Connection.Card->DisableChannel(NTV2Channel(int32_t(InChannel) + ChannelIndex)));
				}

				{
					AJAAutoLock AutoLock(&Connection.ChannelInfoLock);
					ConnectionChannelInfos.erase(FoundIterator);
				}
				delete FoundChannel;

				if (!Connection.Lock_IsInitialize())
				{
					Connection.Lock_UnInitialize();
				}
			}
			else
			{
				if (Connection.bOutputReferenceSet)
				{
					// Connection was also used for genlock but was not initialized as a genlock connection, so mark it as so.
					FoundChannel->bGenlockChannel = true;
				}

				if (bInAsOwner)
				{
					FoundChannel->bIsOwned = false;
				}
			}

			return true;
		}

		bool DeviceConnection::CommandList::IsChannelConnect(NTV2Channel InChannel)
		{
			AJAAutoLock Lock(&Connection.ChannelInfoLock);

			std::vector<ChannelInfo*>& ConnectionChannelInfos = Connection.ChannelInfos;

			auto Found = std::find_if(std::begin(ConnectionChannelInfos), std::end(ConnectionChannelInfos), [=](const ChannelInfo* Info) {return Info->Channel == InChannel; });
			if (Found == std::end(ConnectionChannelInfos))
			{
				return false;
			}
			return (*Found)->bConnected;
		}

		void DeviceConnection::CommandList::SetChannelConnected(NTV2Channel InChannel, bool bConnected)
		{
			AJAAutoLock Lock(&Connection.ChannelLock);

			std::vector<ChannelInfo*>& ConnectionChannelInfos = Connection.ChannelInfos;

			auto Found = std::find_if(std::begin(ConnectionChannelInfos), std::end(ConnectionChannelInfos), [=](const ChannelInfo* Info) {return Info->Channel == InChannel; });
			assert(Found != std::end(ConnectionChannelInfos));
			if (Found != std::end(ConnectionChannelInfos))
			{
				(*Found)->bConnected = bConnected;
			}
		}

		bool DeviceConnection::CommandList::RegisterAnalogLtc(EAnalogLTCSource InSource, NTV2FrameRate InFrameRate, bool bUseReferencePin)
		{
			AJAAutoLock Lock(&Connection.ChannelLock);

			if (!Connection.Lock_IsInitialize() && !Connection.Lock_Initialize())
			{
				return false;
			}
			
			if (!NTV2_IS_VALID_NTV2FrameRate(InFrameRate))
			{
				UE_LOGF(LogAjaCore, Error, "Device: Trying to read LTC from the reference input '%d' on device '%s' but it is not supported by the device.\n", uint32_t(InSource) + 1, Connection.Card->GetDisplayName().c_str());
				return false;
			}

			//Make sure LTC index fits with the device
			bool bResult = true;
			if (bResult && ::NTV2DeviceGetNumLTCInputs(Connection.Card->GetDeviceID()) <= (uint32_t)InSource)
			{
				UE_LOGF(LogAjaCore, Error, "Device: Trying to read LTC from the LTC input '%d' on device '%s' but it is not supported by the device.\n", uint32_t(InSource) + 1, Connection.Card->GetDisplayName().c_str());
				bResult = false;
			}
			
			//If there are other LTC readers, exit, only one LTC input is supported
			if (Connection.LTCFrameRate != NTV2_FRAMERATE_INVALID)
			{
				UE_LOGF(LogAjaCore, Error, "Device: The device '%s' is already reading timecode from LTC input with frame rate '%s' and only one is supported.", Connection.Card->GetDisplayName().c_str(), NTV2FrameRateToString(Connection.LTCFrameRate).c_str());
				bResult = false;
			}

			//When using ref pin, make sure it's not already used
			if (bResult && bUseReferencePin)
			{
				if (!::NTV2DeviceCanDoLTCInOnRefPort(Connection.Card->GetDeviceID()))
				{
					UE_LOGF(LogAjaCore, Error, "Device: Trying to read LTC from reference pin on device '%s' but it doesn't have that capability.\n", Connection.Card->GetDisplayName().c_str());
					bResult = false;
				}

				//Verify if we are already reading LTC from ref pin
				if (bResult && Connection.bAnalogLtcFromReferenceInput)
				{
					UE_LOGF(LogAjaCore, Error, "Device: The reference pin is already used to read LTC timecode on device '%s'.\n", Connection.Card->GetDisplayName().c_str());
					bResult = false;
				}

				//Verify if we are already using ref pin for genlock
				if (bResult && Connection.bOutputReferenceSet && Connection.OutputReferenceType == EAJAReferenceType::EAJA_REFERENCETYPE_EXTERNAL)
				{
					UE_LOGF(LogAjaCore, Error, "Device: Trying to read LTC from the reference input on device '%s' but the reference is already used as a genlock source.\n", Connection.Card->GetDisplayName().c_str());
					bResult = false;
				}
			}

			//Verify if card is used. If not, we need to be sure the card's framerate is configured, especially if channel involved is not SDI1 
			if (bResult)
			{
				auto Found = std::find_if(std::begin(Connection.ChannelInfos), std::end(Connection.ChannelInfos), [=](const ChannelInfo* Info) { return Info->Channel == NTV2_CHANNEL1; });
				if (Found != std::end(Connection.ChannelInfos))
				{
					//LTC reading relies on card/Channel 1 frame rate to be compatible
					const ChannelInfo* FoundChannel = *Found;
					const NTV2FrameRate ChannelFrameRate = ::GetNTV2FrameRateFromVideoFormat(FoundChannel->VideoFormat);
					if (!::IsMultiFormatCompatible(InFrameRate, ChannelFrameRate))
					{
						UE_LOGF(LogAjaCore, Error, "Device: Trying to read LTC with FrameRate '%s' but it's not compatible with card's (channel 1) current FrameRate of '%s' on device '%s'.\n", NTV2FrameRateToString(InFrameRate).c_str(), NTV2FrameRateToString(ChannelFrameRate).c_str(), Connection.Card->GetDisplayName().c_str());
						bResult = false;
					}
				}
				else
				{
					//Nothing set on channel 1, we need to set it to get LTC in
					Connection.Card->SetFrameRate(InFrameRate);
				}
			}

			//Everything went fine, finalize initialization
			if (bResult)
			{
				Connection.LTCFrameRate = InFrameRate;
				
				//If ref pin needs to be use, make sure the card is configured for it. 
				if (bUseReferencePin)
				{
					AJA_CHECK(Connection.Card->SetLTCInputEnable(true));
					Connection.bAnalogLtcFromReferenceInput = true;
				}
			}
			
			if(!bResult && !Connection.Lock_IsInitialize())
			{
				Connection.Lock_UnInitialize();
			}

			return bResult;
		}

		void DeviceConnection::CommandList::UnregisterAnalogLtc(bool bUseReferencePin)
		{
			AJAAutoLock Lock(&Connection.ChannelLock);

			assert(Connection.LTCFrameRate != NTV2_FRAMERATE_INVALID);
			assert(Connection.Lock_IsInitialize());

			//If we are unregistering LTC from ref pin, disable it
			if (bUseReferencePin)
			{
				AJA_CHECK(Connection.Card->SetLTCInputEnable(false));
				Connection.bAnalogLtcFromReferenceInput = false;
			}
			
			Connection.LTCFrameRate = NTV2_FRAMERATE_INVALID;

			if (!Connection.Lock_IsInitialize())
			{
				Connection.Lock_UnInitialize();
			}
		}

		bool DeviceConnection::CommandList::RegisterReference(EAJAReferenceType InOutputReferenceType, NTV2Channel InOutputReferenceChannel)
		{
			AJAAutoLock Lock(&Connection.ChannelLock);

			if (Connection.bOutputReferenceSet)
			{
				if (Connection.OutputReferenceType != InOutputReferenceType)
				{
					UE_LOGF(LogAjaCore, Error, "Device: Couldn't set the reference (%s) for output on device %s. The type was already setup with %s.\n", Helpers::ReferenceTypeToString(InOutputReferenceType), Connection.Card->GetDisplayName().c_str(), Helpers::ReferenceTypeToString(Connection.OutputReferenceType));
					return false;
				}

				if (Connection.OutputReferenceType == EAJAReferenceType::EAJA_REFERENCETYPE_INPUT && Connection.OutputReferenceChannel != InOutputReferenceChannel)
				{
					UE_LOGF(LogAjaCore, Error, "Device: Couldn't set the reference (%s) for output on device %s. The input was already setup with %d.\n", Helpers::ReferenceTypeToString(InOutputReferenceType), Connection.Card->GetDisplayName().c_str(), Connection.OutputReferenceChannel);
					return false;
				}

				return true;
			}

			switch (InOutputReferenceType)
			{
			case EAJAReferenceType::EAJA_REFERENCETYPE_EXTERNAL:
			{
				if (Connection.bAnalogLtcFromReferenceInput)
				{
					UE_LOGF(LogAjaCore, Error, "Device: Couldn't set the reference (%s) for output channel %d on device %s. The reference is used to read analog LTC.\n", Helpers::ReferenceTypeToString(InOutputReferenceType), InOutputReferenceChannel, Connection.Card->GetDisplayName().c_str());
					return false;
				}
				AJA_CHECK(Connection.Card->SetLTCInputEnable(false));
				AJA_CHECK(Connection.Card->SetReference(NTV2_REFERENCE_EXTERNAL));
				break;
			}
			case EAJAReferenceType::EAJA_REFERENCETYPE_INPUT:
			{
				NTV2ReferenceSource Input = NTV2InputSourceToReferenceSource(NTV2ChannelToInputSource(InOutputReferenceChannel));
				if (Input == NTV2_NUM_REFERENCE_INPUTS)
				{
					UE_LOGF(LogAjaCore, Error, "Device: Couldn't set the reference source on device '%s. The Channel index is invalid.", Connection.Card->GetDisplayName().c_str());
					return false;
				}
				AJA_CHECK(Connection.Card->SetReference(Input));
				break;
			}
			case EAJAReferenceType::EAJA_REFERENCETYPE_FREERUN:
			default:
				AJA_CHECK(Connection.Card->SetReference(NTV2_REFERENCE_FREERUN));
			}

			Connection.bOutputReferenceSet = true;
			Connection.OutputReferenceType = InOutputReferenceType;
			Connection.OutputReferenceChannel = InOutputReferenceChannel;

			::Sleep(50 * 1); // Wait 1 frame

			return true;
		}

		void DeviceConnection::CommandList::UnregisterReference(NTV2Channel InChannel)
		{
			AJAAutoLock Lock(&Connection.ChannelLock);
			auto Found = std::find_if(std::begin(Connection.ChannelInfos), std::end(Connection.ChannelInfos), [=](const ChannelInfo* Info) { return Info->Channel == InChannel; });
			
			if (Found == std::end(Connection.ChannelInfos))
			{
				Connection.bOutputReferenceSet = false;
				return;
			}

			const ChannelInfo* FoundChannel = *Found;
			const bool bOutputChannelUsingRefPin = FoundChannel->RefCounter > 1 && Connection.OutputReferenceType == AJA::EAJAReferenceType::EAJA_REFERENCETYPE_EXTERNAL;
			if (!bOutputChannelUsingRefPin)
			{
				Connection.bOutputReferenceSet = false;
			}
		}

		/* Device implementation
		*****************************************************************************/
		CNTV2Card* DeviceConnection::NewCNTV2Card(uint32_t InDeviceIndex)
		{
			CNTV2Card* Result = new CNTV2Card(InDeviceIndex);

			uint32_t Counter = 0;
			while (!Result->IsDeviceReady())
			{
				const DWORD SleepMilli = 10;
				const DWORD TimeoutMilli = 2000;
				if (Counter * SleepMilli > TimeoutMilli)
				{
					UE_LOGF(LogAjaCore, Warning,  "Device: Can't get the device initialized.\n");
					delete Result;
					return nullptr;
				}
				++Counter;
				::Sleep(SleepMilli);
			}

			return Result;
		}

		DeviceConnection::DeviceConnection(const AJADeviceOptions& InDeviceOption)
			: DeviceOption(InDeviceOption)
			, Card(nullptr)
			, TaskMode(NTV2_TASK_MODE_INVALID)
			, bHasBiDirectionalSDI(false)
			, bAnalogLtcFromReferenceInput(false)
			, LTCFrameRate(NTV2FrameRate::NTV2_FRAMERATE_INVALID)
			, bOutputReferenceSet(false)
			, OutputReferenceType(EAJAReferenceType::EAJA_REFERENCETYPE_FREERUN)
			, OutputReferenceChannel(NTV2_CHANNEL_INVALID)
			, NotMultiVideoFormat(NTV2_FORMAT_UNKNOWN)
			, bStopRequested(false)
		{
			CommandThread.Attach(&DeviceConnection::StaticThread, this);
			CommandThread.SetPriority(AJA_ThreadPriority_Normal);
			CommandThread.Start();
		}

		DeviceConnection::~DeviceConnection()
		{
			assert(Card == nullptr);
			assert(ChannelInfos.size() == 0);
			assert(Commands.Size() == 0);

			CommandThread.Kill(-1);
		}

		FDelegateHandle DeviceConnection::SetOnInterruptEvent(NTV2Channel InChannel, AJA::Private::DeviceConnection::FOnVerticalInterrupt::FDelegate OnInterrupt)
		{
			FDelegateHandle DelegateHandle;

			AJAAutoLock Lock(&ChannelInfoLock);
			auto FoundIt = std::find_if(std::begin(ChannelInfos), std::end(ChannelInfos), [=](const ChannelInfo* Info) { return Info->Channel == InChannel; });

			if (FoundIt != std::end(ChannelInfos))
			{
				ChannelInfo* FoundChannel = *FoundIt;

				if (FoundChannel->SyncThread)
				{
					DelegateHandle = FoundChannel->SyncThread->OnVerticalInterrupt().Add(MoveTemp(OnInterrupt));
				}
				else
				{
					UE_LOGF(LogAjaCore, Error, "No sync thread available when calling SetOnInterruptEvent.");
				}
			}

			return DelegateHandle;
		}

		void DeviceConnection::ClearOnInterruptEvent(NTV2Channel InChannel, FDelegateHandle DelegateHandle)
		{
			AJAAutoLock Lock(&ChannelInfoLock);
			auto FoundIt = std::find_if(std::begin(ChannelInfos), std::end(ChannelInfos), [=](const ChannelInfo* Info) { return Info->Channel == InChannel; });
			if (FoundIt == std::end(ChannelInfos))
			{
				return;
			}

			ChannelInfo* FoundChannel = *FoundIt;

			if (FoundChannel->SyncThread)
			{
				FoundChannel->SyncThread->OnVerticalInterrupt().Remove(DelegateHandle);
			}
		}

		bool DeviceConnection::WaitForInputOrOutputInterrupt(NTV2Channel InChannel, int32_t InRepeatCount, EInterruptType InInterrupt)
		{
			// NTV2Card::WaitForInputVerticalInterrupt can wait for only one thread at the same time
			//(if 2 threads wants to wait for the same interrupt on the same channel, only one will be able to wait)
			//This is a wrapper around the wait to enable x threads to wait on the same channel.
			//When more than one thread wants to wait, a special thread is spawn. The special thread will wait and the other thread will wait for the event from the special thread.
			//Once the special thread is spawned, it will stay alive even if only one of the multiple thread is still alive.

			bool bResult = false;

			ChannelInfo* FoundChannel = nullptr;
			{
				AJAAutoLock Lock(&ChannelInfoLock);
				auto Found = std::find_if(std::begin(ChannelInfos), std::end(ChannelInfos), [=](const ChannelInfo* Info) { return Info->Channel == InChannel; });
				if (Found == std::end(ChannelInfos))
				{
					return false;
				}
				FoundChannel = *Found;
			}

			// NTV2Card::WaitForInputVerticalInterrupt will be trigger twice for intercaled, once for progressive and twice for psf.
			// NTV2Card::WaitForInputFieldID will be trigger once for intercaled, once for progressive and once for psf
			//Always wait for WaitForInputVerticalInterrupt and request a second time if InterruptType is InputField.
			//In interlaced, 2 WaitForVerticalInterrupt is required to get a full frame. Wait until the current InputField is the odd/0 field.

			const bool bIsFormatB = ::IsVideoFormatB(FoundChannel->VideoFormat);

			EInterruptType InterruptType = EInterruptType::InputField;
			if (InInterrupt != EInterruptType::Auto)
			{
				InterruptType = InInterrupt;
			}
			else if (bIsFormatB)
			{
				InterruptType = EInterruptType::VerticalInterrupt;
			}

			while (InRepeatCount > 0)
			{
				--InRepeatCount;
				assert(FoundChannel);
				if (FoundChannel->SyncThread)
				{
					bResult = FoundChannel->SyncThread->Wait_ExternalThread();
				}
				else
				{
					bResult = false;
					ensureAlwaysMsgf(false, TEXT("Expected a sync thread but there was none."));
				}

				bool bDoAnotherWaitForField = false;

				if (bResult && InterruptType == EInterruptType::InputField)
				{
					bDoAnotherWaitForField = !SyncThreadInfoThread::IsCurrentField(this, FoundChannel, NTV2_FIELD0);
				}

				if (bDoAnotherWaitForField)
				{
					++InRepeatCount;
				}

				if (bResult && !bDoAnotherWaitForField)
				{
					break;
				}
			}

			return bResult;
		}

		bool DeviceConnection::WaitForInputOrOutputInputField(NTV2Channel InChannel, int32_t InRepeatCount, NTV2FieldID& OutInputField)
		{
			// Similar to WaitForInputOrOutputInterrupt but wait only once in interlaced and return the current InputField

			bool bResult = false;
			OutInputField = NTV2_FIELD_INVALID;

			ChannelInfo* FoundChannel = nullptr;
			{
				AJAAutoLock Lock(&ChannelInfoLock);
				auto Found = std::find_if(std::begin(ChannelInfos), std::end(ChannelInfos), [=](const ChannelInfo* Info) { return Info->Channel == InChannel; });
				if (Found == std::end(ChannelInfos))
				{
					return false;
				}
				FoundChannel = *Found;
			}

			while (InRepeatCount > 0)
			{
				--InRepeatCount;
				assert(FoundChannel);
				if (FoundChannel->SyncThread)
				{
					bResult = FoundChannel->SyncThread->Wait_ExternalThread();
				}
				else
				{
					bResult = false;
					ensureAlwaysMsgf(false, TEXT("Expected a sync thread but there was none."));
				}

				if (bResult)
				{
					OutInputField = SyncThreadInfoThread::IsCurrentField(this, FoundChannel, NTV2_FIELD0) ? NTV2_FIELD0 : NTV2_FIELD1;
					break;

				}
			}

			return bResult;
		}

		NTV2VideoFormat DeviceConnection::GetVideoFormat(NTV2Channel InChannel)
		{
			AJAAutoLock Lock(&ChannelInfoLock);

			auto Found = std::find_if(std::begin(ChannelInfos), std::end(ChannelInfos), [=](const ChannelInfo* Info) {return Info->Channel == InChannel; });
			if (Found == std::end(ChannelInfos))
			{
				return (*Found)->VideoFormat;
			}
			return NTV2VideoFormat::NTV2_FORMAT_UNKNOWN;
		}

		uint32_t DeviceConnection::GetBaseFrameBufferIndex(NTV2Channel InChannel)
		{
			AJAAutoLock Lock(&ChannelInfoLock);

			auto Found = std::find_if(std::begin(ChannelInfos), std::end(ChannelInfos), [=](const ChannelInfo* Info) {return Info->Channel == InChannel; });
			if (Found != std::end(ChannelInfos))
			{
				return (*Found)->BaseFrameBufferIndex;
			}
			return 0;
		}

		bool DeviceConnection::WaitForInputFrameReceived(NTV2Channel InChannel)
		{
			ChannelInfo* FoundChannelInfo = nullptr;
			NTV2VideoFormat VideoFormat = NTV2VideoFormat::NTV2_FORMAT_UNKNOWN;
			{
				AJAAutoLock Lock(&ChannelInfoLock);
				auto Found = std::find_if(std::begin(ChannelInfos), std::end(ChannelInfos), [=](const ChannelInfo* Info) { return Info->Channel == InChannel; });
				if (Found != std::end(ChannelInfos))
				{
					VideoFormat = (*Found)->VideoFormat;
					if ((*Found)->bIsOwned)
					{
						FoundChannelInfo = *Found;
						if (!FoundChannelInfo->WaitForInput)
						{
							FoundChannelInfo->WaitForInput = std::make_unique<WaitForInputInfo>();
						}
					}
				}
			}

			bool bResult = false;
			if (FoundChannelInfo)
			{
				bResult = FoundChannelInfo->WaitForInput->WaitForInput();
			}
			else
			{
				// Wait for 1 field (interlace == 2, progressive == 1). In PSF, we want to wait for 2 fields.
				const DeviceConnection::EInterruptType InterruptType = ::IsPSF(VideoFormat) ? DeviceConnection::EInterruptType::InputField : DeviceConnection::EInterruptType::VerticalInterrupt;
				bResult = WaitForInputOrOutputInterrupt(InChannel, 1, InterruptType);
			}
			return bResult;
		}

		void DeviceConnection::OnInputFrameReceived(NTV2Channel InChannel)
		{
			ChannelInfo* FoundChannelInfo = nullptr;
			{
				AJAAutoLock Lock(&ChannelInfoLock);

				auto Found = std::find_if(std::begin(ChannelInfos), std::end(ChannelInfos), [=](const ChannelInfo* Info) { return Info->Channel == InChannel; });
				if (Found != std::end(ChannelInfos))
				{
					FoundChannelInfo = *Found;
				}
			}

			if (FoundChannelInfo && FoundChannelInfo->WaitForInput)
			{
				FoundChannelInfo->WaitForInput->NotifyAll();
			}
		}

		void DeviceConnection::ClearFormat()
		{
			NotMultiVideoFormat = NTV2_FORMAT_UNKNOWN;
		}

		void DeviceConnection::SetFormat(NTV2Channel InChannel, NTV2VideoFormat InFormat)
		{
			AJAAutoLock Lock(&ChannelLock);
			NotMultiVideoFormat = InFormat;
			// Update the necessary channel
			auto FoundIterator = std::find_if(std::begin(ChannelInfos), std::end(ChannelInfos), [=](const ChannelInfo* Other) { return Other->Channel == InChannel; });
			if (FoundIterator != std::end(ChannelInfos))
			{
				(*FoundIterator)->VideoFormat = InFormat;
			}
		}

		bool DeviceConnection::Lock_EnableChannel_SDILinkHelper(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel) const
		{
			// Check if the Channel match the Transport type.
			const NTV2Channel TransportChannel = Helpers::GetTransportTypeChannel(InTransportType, InChannel);
			const int32_t NumberOfChannels = Helpers::GetNumberOfLinkChannel(InTransportType);
			if (NumberOfChannels != 1)
			{
				if (TransportChannel != InChannel)
				{
					UE_LOGF(LogAjaCore, Error, "Device: Can't do a %s on channel '%d'. Did you mean channel '%d'?"
						, Helpers::TransportTypeToString(InTransportType)
						, uint32_t(InChannel) + 1
						, uint32_t(TransportChannel) + 1);
					return false;
				}
			}

			// ie. if new channel is single 3, need to check that quad 1 is not registered
			// ie. if new channel is quad 1, need to check that single 3 is not registered
			// ie. if new channel is dual 2, need to check that channel 2 is not registered or registered as dual

			const int32_t NewMinChannel = (int32_t)InChannel;
			const int32_t NewMaxChannel = (int32_t)InChannel + Helpers::GetNumberOfLinkChannel(InTransportType) - 1;
			for (ChannelInfo* Itt : ChannelInfos)
			{
				if (Itt->Channel == InChannel)
				{
					if (Itt->TransportType != InTransportType)
					{
						UE_LOGF(LogAjaCore, Error, "Device: Trying to enable the channel '%d' on device '%s' as a '%s' but channel '%d' was already enabled.\n"
							, int32_t(InChannel) + 1
							, InCard->GetDisplayName().c_str()
							, Helpers::TransportTypeToString(InTransportType)
							, int32_t(Itt->Channel) + 1);
						return false;
					}
				}
				else
				{
					const int32_t IttMinChannel = (int32_t)Itt->Channel;
					const int32_t IttMaxChannel = (int32_t)Itt->Channel + Helpers::GetNumberOfLinkChannel(Itt->TransportType) - 1;

					const bool bNewMinInConflict = NewMinChannel >= IttMinChannel && NewMinChannel <= IttMaxChannel;
					const bool bNewMaxInConflict = NewMaxChannel >= IttMinChannel && NewMaxChannel <= IttMaxChannel;
					if (bNewMinInConflict || bNewMaxInConflict)
					{
						UE_LOGF(LogAjaCore, Error, "Device: Trying to enable the channel '%d' on device '%s' as a '%s' but it was already enabled as a '%s' by channel '%d'.\n"
							, uint32_t(InChannel) + 1
							, InCard->GetDisplayName().c_str()
							, Helpers::TransportTypeToString(InTransportType)
							, Helpers::TransportTypeToString(Itt->TransportType)
							, uint32_t(Itt->Channel) + 1);
						return false;
					}
				}
			}

			return true;
		}

		bool DeviceConnection::Lock_EnableChannel_Helper(CNTV2Card* InCard, ETransportType InTransportType, NTV2InputSource InInputSource, NTV2Channel InChannel, bool bIsInput, bool bConnectChannel, bool bIsAutoDetected, ETimecodeFormat InTimecodeFormat, EPixelFormat InUEPixelFormat, NTV2VideoFormat InDesiredInputFormat, NTV2VideoFormat& OutFoundVideoFormat, FAjaHDROptions& InOutHDROptions)
		{
			NTV2DeviceID DeviceId = Card->GetDeviceID();
			if (InChannel >= ::NTV2DeviceGetNumFrameStores(DeviceId))
			{
				UE_LOGF(LogAjaCore, Error, "Device: The device '%s' doesn't support channel '%d'", InCard->GetDisplayName().c_str(), uint32_t(InChannel) + 1);
				return false;
			}

			const int32_t NumberOfLinkChannel = Helpers::GetNumberOfLinkChannel(InTransportType);

			if (bConnectChannel || bIsInput)
			{
				for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
				{
					AJA_CHECK(InCard->EnableChannel(NTV2Channel(int32_t(InChannel) + ChannelIndex)));
				}
			}

			OutFoundVideoFormat = InDesiredInputFormat;

			//If LTC is currently being read, verify frame rate compatibility
			if (LTCFrameRate != NTV2_FRAMERATE_INVALID)
			{
				bool bIsMultiFormatEnabled = false;
				AJA_CHECK(InCard->GetMultiFormatMode(bIsMultiFormatEnabled));
				if (bIsMultiFormatEnabled)
				{
					if (InChannel == NTV2Channel::NTV2_CHANNEL1 && !::IsMultiFormatCompatible(LTCFrameRate, ::GetNTV2FrameRateFromVideoFormat(OutFoundVideoFormat)))
					{
						UE_LOGF(LogAjaCore, Error, "Device: The device '%s' does support Multi Format but channel %d requested format's frame rate ('%s') and it's not compatible with the LTC reference frame rate ('%s').", InCard->GetDisplayName().c_str(), uint32_t(InChannel) + 1, NTV2FrameRateToString(::GetNTV2FrameRateFromVideoFormat(OutFoundVideoFormat)).c_str(), NTV2FrameRateToString(LTCFrameRate).c_str());
						return false;
					}
				}
				else
				{
					if (LTCFrameRate != ::GetNTV2FrameRateFromVideoFormat(OutFoundVideoFormat))
					{
						UE_LOGF(LogAjaCore, Warning,  "Device: The device '%s' doesn't support Multi Format and channel %d requested format's frame rate ('%s') and it's not the same as the LTC reference frame rate ('%s').", InCard->GetDisplayName().c_str(), uint32_t(InChannel) + 1, NTV2FrameRateToString(::GetNTV2FrameRateFromVideoFormat(OutFoundVideoFormat)).c_str(), NTV2FrameRateToString(LTCFrameRate).c_str());
						return false;
					}
				}
			}

			if (NotMultiVideoFormat != NTV2_FORMAT_UNKNOWN && NotMultiVideoFormat != OutFoundVideoFormat)
			{
				bool bIsMultiFormatEnabled = false;
				AJA_CHECK(InCard->GetMultiFormatMode(bIsMultiFormatEnabled));

				if (bIsMultiFormatEnabled)
				{
					if (!::IsMultiFormatCompatible(NotMultiVideoFormat, OutFoundVideoFormat))
					{
						UE_LOGF(LogAjaCore, Error, "Device: The device '%s' does support Multi Format but channel %d requested format ('%s') and it's not compatible with the previous format ('%s').", InCard->GetDisplayName().c_str(), uint32_t(InChannel) + 1, NTV2VideoFormatToString(OutFoundVideoFormat).c_str(), NTV2VideoFormatToString(NotMultiVideoFormat).c_str()); 
						return false;
					}
				}
				else if (!bIsAutoDetected)
				{
					UE_LOGF(LogAjaCore, Error, "Device: The device '%s' doesn't support Multi Format and channel %d requested format ('%s') which is not the same as previous format ('%s').", InCard->GetDisplayName().c_str(), uint32_t(InChannel) + 1, NTV2VideoFormatToString(OutFoundVideoFormat).c_str(), NTV2VideoFormatToString(NotMultiVideoFormat).c_str());
					return false;
				}
			}

			if (InChannel != NTV2_CHANNEL1)
			{
				// Channel 1 dictates the card's framerate, so make sure we are running at a compatible framerate.
				NTV2FrameRate DeviceFramerate = NTV2_FRAMERATE_INVALID;
				AJA_CHECK(InCard->GetFrameRate(DeviceFramerate));

				NTV2FrameRate NewVideoFormatFrameRate = GetNTV2FrameRateFromVideoFormat(OutFoundVideoFormat);

				NTV2FrameRate NewVideoFormatFrameRateFamily = GetFrameRateFamily(NewVideoFormatFrameRate);

				if (GetFrameRateFamily(DeviceFramerate) != NewVideoFormatFrameRateFamily)
				{
					// Device framerate doesn't match... We'll override it if the channel isn't in use.
					AJAAutoLock AutoLock(&ChannelInfoLock);

					// Check all channels that are already in use before changing the framerate.
					// We can't change the card framerate if it's driven by a different channel.
					for (ChannelInfo* Info : ChannelInfos)
					{
						if (Info)
						{
							NTV2FrameRate ChannelVideoFormatFrameRate = GetNTV2FrameRateFromVideoFormat(Info->VideoFormat);
							if (NewVideoFormatFrameRateFamily != GetFrameRateFamily(ChannelVideoFormatFrameRate))
							{
								UE_LOGF(LogAjaCore, Error, "Device: Can't enable channel %d device '%s' because its framerate doesn't match an existing channel's framerate family.", uint32_t(InChannel) + 1, InCard->GetDisplayName().c_str());
								return false;
							}
						}
					}

					AJA_CHECK(InCard->SetFrameRate(NewVideoFormatFrameRate));
				}
			}


			NotMultiVideoFormat = OutFoundVideoFormat;

			const bool bIsSDI = Helpers::IsSdiTransport(InTransportType);
			const bool bIsHDMI = Helpers::IsHdmiTransport(InTransportType);

			//	Disable SDI output from the SDI input being used,
			//	but only if the device supports bi-directional SDI,
			//	and only if the input being used is an SDI input...

			//	If the device supports bi-directional SDI and the
			//	requested input is SDI, ensure the SDI direction
			//	is configured for input...
			if (bHasBiDirectionalSDI && bIsSDI)
			{
				for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
				{
					AJA_CHECK(InCard->SetSDITransmitEnable(NTV2Channel(int32_t(InChannel) + ChannelIndex), !bIsInput));
				}
				AJATime::Sleep(50 * 12); //	...and give the device 12 frames or so to lock to the input signal
			}

			if (bIsInput && bIsSDI)
			{
				std::string FailureReason;
				if (!Helpers::GetInputVideoFormat(InCard, InTransportType, InChannel, InInputSource, InDesiredInputFormat, OutFoundVideoFormat, true, FailureReason))
				{
					UE_LOGF(LogAjaCore, Error, "Device: Initialization of the input failed for channel %d on device '%s'. %s", uint32_t(InChannel) + 1, InCard->GetDisplayName().c_str(), FailureReason.c_str());
					return false;
				}
				
				if (!Helpers::GetInputHDRMetadata(InCard, InChannel, InOutHDROptions))
				{
					UE_LOGF(LogAjaCore, Error, "Device: Could not fetch HDR metadata from channel %d on device '%s'.", uint32_t(InChannel) + 1, InCard->GetDisplayName().c_str());
					return false;
				}
			}

			// Setup basic video to support Interrupt. This will be override by ChannelBase if necessary
			if (bIsInput)
			{
				//	Set the frame buffer pixel format for all the channels on the device
				for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
				{
					AJA_CHECK(InCard->SetMode(NTV2Channel(int32_t(InChannel) + ChannelIndex), NTV2_MODE_CAPTURE));
				}
			}
			else
			{
				//	Set the frame buffer pixel format for all the channels on the device
				for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
				{
					AJA_CHECK(InCard->SetMode(NTV2Channel(int32_t(InChannel) + ChannelIndex), NTV2_MODE_DISPLAY));
				}
			}

			const NTV2FrameBufferFormat FrameBufferFormat = Helpers::ConvertPixelFormatToFrameBufferFormat(InUEPixelFormat);
			AJA_CHECK(InCard->SetVideoFormat(OutFoundVideoFormat, AJA_RETAIL_DEFAULT, false, InChannel));

			const NTV2HDRXferChars EOTF = Helpers::ConvertToAjaHDRXferChars(InOutHDROptions.EOTF);
			const NTV2HDRColorimetry Colorimetry = Helpers::ConvertToAjaHDRColorimetry(InOutHDROptions.Gamut);
			const NTV2HDRLuminance Luminance = Helpers::ConvertToAjaHDRLuminance(InOutHDROptions.Luminance);
			
			for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
			{
				AJA_CHECK(InCard->SetFrameBufferFormat(NTV2Channel(int32_t(InChannel) + ChannelIndex), FrameBufferFormat, AJA_RETAIL_DEFAULT, EOTF, Colorimetry, Luminance));
			}

			// Route the input for the sync, seems required
			if (bConnectChannel)
			{
				if (bIsSDI)
				{
					const bool bIsSignalRgb = false;
					const bool bWillUseKey = false;
					Helpers::RouteSdiSignal(InCard, InTransportType, InChannel, OutFoundVideoFormat, FrameBufferFormat, bIsInput, bIsSignalRgb, bWillUseKey);
				}
				else if (bIsHDMI)
				{
					Helpers::RouteHdmiSignal(InCard, InTransportType, InInputSource, InChannel, FrameBufferFormat, bIsInput);
				}
			}
			else
			{
				// route basic output for the sync, seems required for SyncChannel
				if (bIsSDI && bIsInput)
				{
					const bool bIsDS2 = false;
					AJA_CHECK(InCard->Connect(::GetFrameBufferInputXptFromChannel(InChannel), ::GetSDIInputOutputXptFromChannel(InChannel, bIsDS2)));
				}
				else if (bIsSDI && !bIsInput)
				{
					AJA_CHECK(InCard->Connect(::GetSDIOutputInputXpt(InChannel), ::GetFrameBufferOutputXptFromChannel(InChannel)));
				}
				else if (bIsHDMI && bIsInput)
				{
					AJA_CHECK(InCard->Connect(::GetFrameBufferInputXptFromChannel(InChannel), ::GetInputSourceOutputXpt(::GetNTV2HDMIInputSourceForIndex((ULWord)InChannel))));
				}
				else if (bIsHDMI && !bIsInput)
				{
					AJA_CHECK(InCard->Connect(::GetOutputDestInputXpt(NTV2_OUTPUTDESTINATION_HDMI), ::GetFrameBufferOutputXptFromChannel(InChannel)));
				}
			}

			// Tell if it's dual or Quad SQ or Quad SI
			if (Helpers::IsTsiRouting(InTransportType))
			{
				AJA_CHECK(InCard->SetTsiFrameEnable(true, InChannel));
			}
			else if (InTransportType == ETransportType::TT_SdiDual)
			{
				AJA_CHECK(InCard->SetSmpte372(true, InChannel));
			}
			else if (InTransportType == ETransportType::TT_SdiQuadSQ)
			{
				AJA_CHECK(InCard->Set4kSquaresEnable(true, InChannel));
			}

			if (bIsInput && bIsHDMI)
			{
				std::string FailureReason;
				if (!Helpers::GetInputVideoFormat(InCard, InTransportType, InChannel, InInputSource, InDesiredInputFormat, OutFoundVideoFormat, true, FailureReason))
				{
					UE_LOGF(LogAjaCore, Error, "Device: Initialization of the input failed for channel %d on device '%s'. %s", uint32_t(InChannel) + 1, InCard->GetDisplayName().c_str(), FailureReason.c_str());
					return false;
				}
			}

			if (InTimecodeFormat != ETimecodeFormat::TCF_None)
			{
				//	Be sure the RP188 source for the input channel is being grabbed from the right input...
				const ULWord TimecodeFilter = 0xFF; //No filter. We filter out once we read it
				AJA_CHECK(InCard->SetRP188SourceFilter(InChannel, TimecodeFilter));
				if (bIsInput)
				{
					AJA_CHECK(InCard->SetRP188Mode(InChannel, NTV2_RP188_INPUT));
				}
				else
				{
					AJA_CHECK(InCard->SetRP188Mode(InChannel, NTV2_RP188_OUTPUT));
				}
			}

			return true;
		}

		void DeviceConnection::Lock_SubscribeChannel_Helper(CNTV2Card* InCard, NTV2Channel InChannel, bool InAsInput) const
		{
			if (InAsInput)
			{
				AJA_CHECK(InCard->EnableInputInterrupt(InChannel));
				AJA_CHECK(InCard->SubscribeInputVerticalEvent(InChannel));
			}
			else
			{
				AJA_CHECK(InCard->EnableOutputInterrupt(InChannel));
				AJA_CHECK(InCard->SubscribeOutputVerticalEvent(InChannel));
			}
		}

		bool DeviceConnection::Lock_Initialize()
		{
			assert(Card == nullptr);

			Card = NewCNTV2Card(DeviceOption.DeviceIndex);

			//Get the application currently owning the Aja device, if any
			ULWord   CurrentAppFourCC(AJA_FOURCC('?', '?', '?', '?'));
			int32_t  CurrentAppPID(0);
			if (!Card->GetStreamingApplication(&CurrentAppFourCC, &CurrentAppPID))
			{
				UE_LOGF(LogAjaCore, Error, "Device: Couldn't get current stream using Aja. Something is wrong when tryin to access registers on device '%s'.", Card->GetDisplayName().c_str());
				delete Card;
				Card = nullptr;
				return false;
			}

			//Try acquiring the card. If a crashed process still has a hook on it, we'll be able to take over. 
			if (!Card->AcquireStreamForApplication(UEAppType, static_cast<uint32_t>(AJAProcess::GetPid())))
			{
				//If there was a process owning it and we were not able to override it, it must still be active. Notify the user who owns it.
				if (CurrentAppPID)
				{
					char FourCCBuffer[5];
#if defined(AJA_LITTLE_ENDIAN)
					FourCCBuffer[0] = reinterpret_cast<const char*>(&CurrentAppFourCC)[3];
					FourCCBuffer[1] = reinterpret_cast<const char*>(&CurrentAppFourCC)[2];
					FourCCBuffer[2] = reinterpret_cast<const char*>(&CurrentAppFourCC)[1];
					FourCCBuffer[3] = reinterpret_cast<const char*>(&CurrentAppFourCC)[0];
#else
					FourCCBuffer[0] = reinterpret_cast<const char*>(&CurrentAppFourCC)[0];
					FourCCBuffer[1] = reinterpret_cast<const char*>(&CurrentAppFourCC)[1];
					FourCCBuffer[2] = reinterpret_cast<const char*>(&CurrentAppFourCC)[2];
					FourCCBuffer[3] = reinterpret_cast<const char*>(&CurrentAppFourCC)[3];
#endif
					FourCCBuffer[4] = '\0';

					UE_LOGF(LogAjaCore, Error, "Device: Couldn't acquire stream for Unreal Engine application on device '%s'. Used by application '%s' with PID '%d'.", Card->GetDisplayName().c_str(), FourCCBuffer, CurrentAppPID);
				}
				else
				{
					UE_LOGF(LogAjaCore, Error, "Device: Couldn't acquire stream for Unreal Engine application on device '%s'.", Card->GetDisplayName().c_str());
				}

				delete Card;
				Card = nullptr;
				return false;
			}
			
			{
				UE_LOGF(LogAjaCore,  Display, "Connected to %s with SDK %d.%d.%d.%d", Card->GetDeviceVersionString().c_str(), AJA_NTV2_SDK_VERSION_MAJOR, AJA_NTV2_SDK_VERSION_MINOR, AJA_NTV2_SDK_VERSION_POINT, AJA_NTV2_SDK_BUILD_NUMBER);
				if (AJA_NTV2_SDK_VERSION_MAJOR < 14)
				{
					UE_LOGF(LogAjaCore, Warning,  "Unreal Engine's implementation of AJA support SDK 14 and up.");
				}
			}

			// save the service level if it was not an unreleased UE app owning the card
			TaskMode = NTV2_STANDARD_TASKS;
			if (CurrentAppFourCC != UEAppType)
			{
				Card->GetEveryFrameServices(TaskMode);
			}

			// Use the OEM service level.
			Card->SetEveryFrameServices(NTV2_OEM_TASKS);
			// clear the routing
			Card->ClearRouting();

			::Sleep(50*1); // Wait 1 frame

			NTV2DeviceID DeviceId = Card->GetDeviceID();
			bHasBiDirectionalSDI = ::NTV2DeviceHasBiDirectionalSDI(DeviceId);

			if (DeviceOption.bWantMultiFormatMode)
			{
				if (::NTV2DeviceCanDoMultiFormat(Card->GetDeviceID()))
				{
					bool bEnabled = true;
					Card->SetMultiFormatMode(bEnabled);
				}
				else
				{
					UE_LOGF(LogAjaCore, Warning,  "Device: The device '%s' doesn't support Multi Format.", Card->GetDisplayName().c_str());
				}
			}

			return true;
		}

		void DeviceConnection::Lock_UnInitialize()
		{
			Card->SetEveryFrameServices(TaskMode);
			Card->ReleaseStreamForApplication(UEAppType, static_cast<uint32_t>(AJAProcess::GetPid()));
			delete Card;
			Card = nullptr;
		}

		uint32_t DeviceConnection::Lock_AcquireBaseFrameIndex(NTV2Channel InChannel, NTV2VideoFormat InVideoFormat) const
		{
			const uint32_t MaxValue = ::NTV2DeviceGetNumberFrameBuffers(Card->GetDeviceID());
			const uint32_t BaseFrameIndex = InChannel * NumberOfFrameToAquire * 4;

			if (BaseFrameIndex < MaxValue)
			{
				return BaseFrameIndex;
			}
			else
			{
				UE_LOGF(LogAjaCore, Error, "Device: Too many Frames Indexes requested on device '%s'.\n", Card->GetDisplayName().c_str());
				return InvalidFrameBufferIndex;
			}
		}

		void DeviceConnection::AddCommand(const std::shared_ptr<DeviceCommand>& Command)
		{
			Commands.Push(Command);
			CommandEvent.Signal();
		}

		void DeviceConnection::RemoveCommand(const std::shared_ptr<DeviceCommand>& Command)
		{
			Commands.Erase(Command);
		}

		void DeviceConnection::StaticThread(AJAThread* pThread, void* pContext)
		{
			DeviceConnection* DeviceContext = reinterpret_cast<DeviceConnection*>(pContext);
			DeviceContext->Thread_CommandLoop();
		}

		void DeviceConnection::Thread_CommandLoop()
		{
			while (!bStopRequested)
			{
				// It's a manual signal. Will stay opened until cleared.
				CommandEvent.WaitForSignal();
				CommandEvent.Clear();

				if (bStopRequested)
				{
					return;
				}

				while (true)
				{
					std::shared_ptr<DeviceCommand> NextCommand;
					if (!Commands.Pop(NextCommand) && !NextCommand)
					{	// wait for next signal
						break;
					}

					DeviceConnection::CommandList Command(*this);
					NextCommand->Execute(Command);

					if (bStopRequested)
					{
						return;
					}
				}
			}
		}

		/* DeviceCommand implementation
		*****************************************************************************/
		DeviceCommand::DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection)
			: Device(InConnection)
		{
		}
	}
}
