// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Helpers.h"
#include "ThreadSafeQueue.h"

#include <atomic>
#include <condition_variable>
#include <memory>
#include <mutex>
#include <vector>

namespace AJA
{
	namespace Private
	{
		class DeviceConnection;
		class DeviceCache;
		struct DeviceCommand;

		/**
		 * The Device will Initialize itself. There is no callback to tell the user the Device is created properly.
		 * It should only be call from AJA thread.
		 * Other Channel will register and lock InitializationLock.
		 * The fist RegisterAndLockChannel will lock and complete the initialization.
		 */
		class DeviceConnection
		{
		public:
			static const uint32_t NumberOfFrameToAquire = 4;
			static const uint32_t NumberOfFrameForAutoCirculate = NumberOfFrameToAquire;
			static const uint32_t InvalidFrameBufferIndex = 0xFFFFFFFF;
			DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnVerticalInterrupt, uint32 /*NewSyncCount*/);

		private:
			struct ChannelInfo;
			class SyncThreadInfoThread;

			class SyncThreadInfoThread : public AJAThread
			{
			public:
				SyncThreadInfoThread(DeviceConnection* InDeviceConnection, ChannelInfo* InChannelInfo);
				/** Delegate triggered whenever the VBI count has increased. */
				FOnVerticalInterrupt& OnVerticalInterrupt() { return OnVerticalInterruptDelegate; }

			private:
				ChannelInfo* ChannelInfoPtr;
				DeviceConnection* DeviceConnectionPtr;

				std::mutex SyncMutex;
				std::condition_variable SyncCondition;
				volatile bool bWaitResult;
				FOnVerticalInterrupt OnVerticalInterruptDelegate;

			public:
				bool WaitForVerticalInterrupt(DeviceConnection* InDeviceConnection, ChannelInfo* InChannelInfo, uint32& OutSyncCount);
				static bool IsCurrentField(DeviceConnection* InDeviceConnection, ChannelInfo* InChannelInfo, NTV2FieldID InFieldId = NTV2_FIELD0);
				bool Wait_ExternalThread();
				virtual bool ThreadLoop() override;
			};

			class WaitForInputInfo
			{
			private:
				std::mutex SyncMutex;
				std::condition_variable SyncCondition;

			public:
				bool WaitForInput();
				void NotifyAll();
			};

			struct ChannelInfo
			{
				ChannelInfo();

				// If the ref count > 1, a SyncThread will be spawn. See DeviceConnection::WaitForVerticalInterrupt.
				std::unique_ptr<SyncThreadInfoThread> SyncThread;
				std::unique_ptr<WaitForInputInfo> WaitForInput;
				ETransportType TransportType;
				NTV2Channel Channel;
				NTV2VideoFormat VideoFormat;
				ETimecodeFormat TimecodeFormat;
				int32_t RefCounter;
				uint32_t BaseFrameBufferIndex;
				bool bConnected;
				bool bIsOwned;
				bool bIsInput;
				bool bIsAutoDetected;
				bool bGenlockChannel;
			};

		public:
			DeviceConnection() = delete;
			DeviceConnection(const AJADeviceOptions& InDeviceOption);
			~DeviceConnection();
			DeviceConnection(const DeviceConnection&) = delete;
			DeviceConnection& operator=(const DeviceConnection&) = delete;

			static CNTV2Card* NewCNTV2Card(uint32_t InDeviceIndex);

			enum class EInterruptType
			{
				VerticalInterrupt,
				InputField,
				Auto,
			};

			/**
			 * Set a callback to be invoked whenever the card sends a vertical interrupt signal.
			 * @return a valid Delegate if the delegate was set correctly.
			 */
			FDelegateHandle SetOnInterruptEvent(NTV2Channel InChannel, AJA::Private::DeviceConnection::FOnVerticalInterrupt::FDelegate OnInterrupt);
			void ClearOnInterruptEvent(NTV2Channel InChannel, FDelegateHandle DelegateHandle);
			bool WaitForInputOrOutputInterrupt(NTV2Channel InChannel, int32_t InRepeatCount = 1, EInterruptType InInterrupt = EInterruptType::Auto);
			bool WaitForInputOrOutputInputField(NTV2Channel InChannel, int32_t InRepeatCount, NTV2FieldID& OutInputField);
			NTV2VideoFormat GetVideoFormat(NTV2Channel InChannel);
			uint32_t GetBaseFrameBufferIndex(NTV2Channel InChannel);
			bool WaitForInputFrameReceived(NTV2Channel InChannel);
			void OnInputFrameReceived(NTV2Channel InChannel);
			CNTV2Card* GetCard() { return Card; }
			void ClearFormat();
			void SetFormat(NTV2Channel InChannel, NTV2VideoFormat InFormat);

		public:
			struct CommandList
			{
			public:
				CommandList(DeviceConnection&);

			public:
				bool RegisterChannel(ETransportType InTransportType, NTV2InputSource InInputSource, NTV2Channel InChannel, bool bInAsInput, bool bInAsGenlock, bool bConnectChannel, ETimecodeFormat InTimecodeFormat, EPixelFormat InUEPixelFormat, NTV2VideoFormat InDesiredInputFormat, bool bInAsOwner, bool bInIsAutoDetected);
				/** RegisterChannel override that handles HDR metadata. */
				bool RegisterChannel(ETransportType InTransportType, NTV2InputSource InInputSource, NTV2Channel InChannel, bool bInAsInput, bool bInAsGenlock, bool bConnectChannel, ETimecodeFormat InTimecodeFormat, EPixelFormat InUEPixelFormat, NTV2VideoFormat InDesiredInputFormat, FAjaHDROptions& InOutHDROptions, bool bInAsOwner, bool bInIsAutoDetected);
				bool UnregisterChannel(NTV2Channel InChannel, bool bInAsOwner);

				bool IsChannelConnect(NTV2Channel InChannel);
				void SetChannelConnected(NTV2Channel InChannel, bool bConnected);

				bool RegisterAnalogLtc(EAnalogLTCSource InSource, NTV2FrameRate InFrameRate, bool bUseReferencePin);
				void UnregisterAnalogLtc(bool bUseReferencePin);

				bool RegisterReference(EAJAReferenceType OutputReferenceType, NTV2Channel InChannel);
				void UnregisterReference(NTV2Channel InChannel);

			private:
				DeviceConnection& Connection;
			};
			friend CommandList;

		private:
			friend DeviceCache;
			bool Lock_IsInitialize() const { return ChannelInfos.size() != 0 || LTCFrameRate != NTV2_FRAMERATE_INVALID; }
			bool Lock_Initialize();
			void Lock_UnInitialize();

			bool Lock_EnableChannel_SDILinkHelper(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel) const;
			bool Lock_EnableChannel_Helper(CNTV2Card* InCard, ETransportType InTransportType, NTV2InputSource InInputSource, NTV2Channel InChannel, bool InAsInput, bool bConnectChannel, bool bIsAutoDetected, ETimecodeFormat InTimecodeFormat, EPixelFormat InUEPixelFormat, NTV2VideoFormat InDesiredInputFormat, NTV2VideoFormat& OutFoundVideoFormat, FAjaHDROptions& InOutHDROptions);
			void Lock_SubscribeChannel_Helper(CNTV2Card* InCard, NTV2Channel InChannel, bool InAsInput) const;

			uint32_t Lock_AcquireBaseFrameIndex(NTV2Channel InChannel, NTV2VideoFormat InVideoFormat) const;

		private:
			AJADeviceOptions DeviceOption;

			// Lock when we want to read and we are not under the ChannelLock. Use ChannelLock in the initialization phase.
			AJALock ChannelInfoLock;
			std::vector<ChannelInfo*> ChannelInfos;

			CNTV2Card* Card;
			NTV2EveryFrameTaskMode TaskMode;
			bool bHasBiDirectionalSDI;

			//Is device's reference pin currently used to read LTC
			bool bAnalogLtcFromReferenceInput;
			NTV2FrameRate LTCFrameRate;

			bool bOutputReferenceSet;
			EAJAReferenceType OutputReferenceType;
			NTV2Channel OutputReferenceChannel;
			NTV2VideoFormat NotMultiVideoFormat;

		public:
			void AddCommand(const std::shared_ptr<DeviceCommand>& Command);
			void RemoveCommand(const std::shared_ptr<DeviceCommand>& Command);

		private:
			AJALock ChannelLock;
			AJAEvent CommandEvent;
			ThreadSafeQueue<std::shared_ptr<DeviceCommand>> Commands;
			volatile bool bStopRequested;
			AJAThread CommandThread;

		private:
			static void StaticThread(AJAThread* pThread, void* pContext);
			void Thread_CommandLoop();
		};

		/* DeviceCommand definition
		*****************************************************************************/
		struct DeviceCommand
		{
			DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection);
			virtual ~DeviceCommand() {}

			virtual void Execute(DeviceConnection::CommandList& InCommandList) = 0;

			std::shared_ptr<DeviceConnection> Device;
		};

		/* DeviceCache definition
		*****************************************************************************/
		class DeviceCache
		{
		public:
			static const uint32_t MaxNumberOfDevice = 8;

			// This is a slow operation. Should be call from a thread
			static std::shared_ptr<DeviceConnection> GetDevice(const AJADeviceOptions& InDeviceOption);
			static void RemoveDevice(const std::shared_ptr<DeviceConnection>& InDevice);

		private:
			AJALock Lock;
			std::weak_ptr<DeviceConnection> DeviceInstance[MaxNumberOfDevice];
		};
	}
}
