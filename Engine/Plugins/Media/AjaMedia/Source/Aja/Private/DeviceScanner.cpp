// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeviceScanner.h"

#include <algorithm>
#include <string>


namespace AJA
{
	namespace Private
	{
		/* DeviceScanner implementation
		*****************************************************************************/
		DeviceScanner::DeviceScanner()
		{
			Scanner = new CNTV2DeviceScanner();
			Scanner->ScanHardware();
		}

		DeviceScanner::~DeviceScanner()
		{
			delete Scanner;
		}

		int32_t DeviceScanner::GetNumDevices() const
		{
			return (int32_t)Scanner->GetNumDevices();
		}

		bool DeviceScanner::GetDeviceTextId(int32_t InDeviceIndex, AJADeviceScanner::FormatedTextType& OutTextId) const
		{
			NTV2DeviceInfo DeviceInfo;
			if (!Scanner->GetDeviceInfo(InDeviceIndex, DeviceInfo, false))
			{
				OutTextId[0] = TEXT('\0');
				return false;
			}

			std::string DeviceName = NTV2DeviceIDToString(DeviceInfo.deviceID);
			MultiByteToWideChar(CP_UTF8, 0, &DeviceName[0], (int32_t)DeviceName.size() + 1, OutTextId, AJADeviceScanner::FormatedTextSize);
			return true;
		}

		bool DeviceScanner::GetDeviceInfo(int32_t InDeviceIndex, AJADeviceScanner::DeviceInfo& OutDeviceInfo) const
		{
			NTV2DeviceInfo DeviceInfo;
			if (!Scanner->GetDeviceInfo(InDeviceIndex, DeviceInfo, false))
			{
				return false;
			}

			// We do not support UFC because input 0 != output 0 (need to code it, so that when we talk about output 0, we actually talk about channel 3)
			const bool bSupportedDeviceByUE = DeviceInfo.deviceID != DEVICE_ID_KONA4UFC;
			const bool bIsSupportedBySDK = ::NTV2DeviceIsSupported(DeviceInfo.deviceID);
			const bool bCanDoMessageSignaledInterrupts = ::NTV2DeviceCanDoMSI(DeviceInfo.deviceID);

			OutDeviceInfo.bIsSupported = bIsSupportedBySDK && bCanDoMessageSignaledInterrupts && bSupportedDeviceByUE;

			if (OutDeviceInfo.bIsSupported)
			{
				UWord NumFrameStores = ::NTV2DeviceGetNumFrameStores(DeviceInfo.deviceID);
				OutDeviceInfo.NumSdiInput = FMath::Min(DeviceInfo.numVidInputs, NumFrameStores);
				OutDeviceInfo.NumSdiOutput = FMath::Min(DeviceInfo.numVidOutputs, NumFrameStores);
				OutDeviceInfo.NumHdmiInput = FMath::Min(DeviceInfo.numHDMIVidInputs, NumFrameStores);
				OutDeviceInfo.NumHdmiOutput = FMath::Min(DeviceInfo.numHDMIVidOutputs, NumFrameStores);
			}

			if (OutDeviceInfo.bIsSupported)
			{
				OutDeviceInfo.bIsSupported = OutDeviceInfo.NumSdiInput > 0 || OutDeviceInfo.NumHdmiInput > 0 || OutDeviceInfo.NumSdiOutput > 0;
			}

			if (OutDeviceInfo.bIsSupported)
			{
				OutDeviceInfo.bCanFrameStore1DoPlayback = ::NTV2DeviceCanDoFrameStore1Display(DeviceInfo.deviceID);
				OutDeviceInfo.bCanDoDualLink = DeviceInfo.dualLinkSupport;
				OutDeviceInfo.bCanDo2K = DeviceInfo.has2KSupport;
				OutDeviceInfo.bCanDo12GSdi = ::NTV2DeviceCanDo12GSDI(DeviceInfo.deviceID);
				OutDeviceInfo.bCanDo12GRouting = ::NTV2DeviceCanDo12gRouting(DeviceInfo.deviceID); //DEVICE_ID_KONA5_12G
				OutDeviceInfo.bCanDoMultiFormat = DeviceInfo.multiFormat;
				OutDeviceInfo.bCanDoAlpha = DeviceInfo.rgbAlphaOutputSupport; // The CSC can split the alpha
				OutDeviceInfo.bCanDo3GLevelConversion = ::NTV2DeviceCanDo3GLevelConversion(DeviceInfo.deviceID);
				OutDeviceInfo.bCanDo4K = DeviceInfo.has4KSupport;
				OutDeviceInfo.bCanDoTSI = DeviceInfo.has4KSupport && DeviceInfo.deviceID != DEVICE_ID_KONA5_4X12G;

				OutDeviceInfo.bCanDoCustomAnc = ::NTV2DeviceCanDoCustomAnc(DeviceInfo.deviceID);
				OutDeviceInfo.bCanDoLtc = ::NTV2DeviceCanDoLTC(DeviceInfo.deviceID);
				OutDeviceInfo.bCanDoLtcInRefPort = ::NTV2DeviceCanDoLTCInOnRefPort(DeviceInfo.deviceID);
				OutDeviceInfo.NumberOfLtcInput = ::NTV2DeviceGetNumLTCInputs(DeviceInfo.deviceID);
				OutDeviceInfo.NumberOfLtcOutput = ::NTV2DeviceGetNumLTCOutputs(DeviceInfo.deviceID);

				OutDeviceInfo.bSupportPixelFormat8bitYCBCR = ::NTV2DeviceCanDoFrameBufferFormat(DeviceInfo.deviceID, Helpers::ConvertPixelFormatToFrameBufferFormat(EPixelFormat::PF_8BIT_YCBCR));
				OutDeviceInfo.bSupportPixelFormat8bitARGB = ::NTV2DeviceCanDoFrameBufferFormat(DeviceInfo.deviceID, Helpers::ConvertPixelFormatToFrameBufferFormat(EPixelFormat::PF_8BIT_ARGB));
				OutDeviceInfo.bSupportPixelFormat10bitRGB = ::NTV2DeviceCanDoFrameBufferFormat(DeviceInfo.deviceID, Helpers::ConvertPixelFormatToFrameBufferFormat(EPixelFormat::PF_10BIT_RGB));
				OutDeviceInfo.bSupportPixelFormat10bitYCBCR = ::NTV2DeviceCanDoFrameBufferFormat(DeviceInfo.deviceID, Helpers::ConvertPixelFormatToFrameBufferFormat(EPixelFormat::PF_10BIT_YCBCR));

				if (OutDeviceInfo.NumSdiInput > 0 && !DeviceInfo.programmableCSCSupport)
				{
					OutDeviceInfo.bSupportPixelFormat8bitARGB = false;
					OutDeviceInfo.bSupportPixelFormat10bitRGB = false;
				}
			}

			return true;
		}
	}
}
