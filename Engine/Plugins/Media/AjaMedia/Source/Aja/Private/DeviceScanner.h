// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers.h"

#include <vector>


namespace AJA
{
	namespace Private
	{
		/* DeviceScanner definition
		*****************************************************************************/
		class DeviceScanner
		{
		public:
			DeviceScanner();
			~DeviceScanner();

			DeviceScanner(const DeviceScanner&) = delete;
			DeviceScanner& operator=(const DeviceScanner&) = delete;

			int32_t GetNumDevices() const;
			bool GetDeviceTextId(int32_t InDeviceIndex, AJADeviceScanner::FormatedTextType& OutTextId) const;
			bool GetDeviceInfo(int32_t InDeviceIndex, AJADeviceScanner::DeviceInfo& OutDeviceInfo) const;

		private:
			CNTV2DeviceScanner* Scanner;
		};
	}
}
