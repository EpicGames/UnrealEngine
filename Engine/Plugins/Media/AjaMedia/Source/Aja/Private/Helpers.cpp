// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers.h"

namespace AJA
{
	namespace Private
	{
		FTimecode Helpers::ConvertTimecodeFromRP188(const RP188_STRUCT& InRP188, const NTV2VideoFormat InVideoFormat)
		{
			NTV2_RP188 Value(InRP188);
			return ConvertTimecodeFromRP188(Value, InVideoFormat);
		}

		FTimecode Helpers::ConvertTimecodeFromRP188(const NTV2_RP188& InRP188, const NTV2VideoFormat InVideoFormat)
		{
			FTimecode Result;

			const TimecodeFormat TcFormat = ConvertToTimecodeFormat(InVideoFormat);

			if (InRP188.fDBB == 0xffffffff)
				return Result;

			ULWord TC0_31 = InRP188.fLo;
			ULWord TC32_63 = InRP188.fHi;

			const bool _bDropFrameFlag = (TC0_31 >> 10) & 0x01;	// Drop Frame:  timecode bit 10
			const bool bIsGreaterThan30 = (TcFormat == kTCFormat60fps || TcFormat == kTCFormat60fpsDF || TcFormat == kTCFormat48fps || TcFormat == kTCFormat50fps);
			const bool bIsPal = (TcFormat == kTCFormat25fps || TcFormat == kTCFormat50fps);
			const bool bIsInterlaced = !::IsProgressivePicture(InVideoFormat);
			const bool bFieldID = (bIsPal ? ((TC32_63 & BIT_27) != 0) : ((TC0_31 & BIT_27) != 0));	// Note: FID is in different words for PAL & NTSC!

			static const int8_t bcd[] = { '0', '1', '2', '3', '4', '5', '6', '7', '8', '9', '0', '0', '0', '0', '0', '0' };

			int8_t UnitFrames;
			int8_t TensFrames;
			if (bIsGreaterThan30 || bIsInterlaced)
			{
				// for frame rates > 30 fps, the field ID correlates frame pairs. 
				const int32_t NumFrames = (((((TC0_31 >> 8) & 0x3) * 10) + (TC0_31 & 0xF)) * 2); // double the regular frame count and don't add field id. We manage this manually.
				UnitFrames = bcd[NumFrames % 10];
				TensFrames = bcd[NumFrames / 10];
			}
			else
			{
				UnitFrames = bcd[(TC0_31) & 0xF];
				TensFrames = bcd[(TC0_31 >> 8) & 0x3];
			}
			int8_t UnitSeconds = bcd[(TC0_31 >> 16) & 0xF];
			int8_t TensSeconds = bcd[(TC0_31 >> 24) & 0x7];

			int8_t UnitMinutes = bcd[(TC32_63) & 0xF];
			int8_t TensMinutes = bcd[(TC32_63 >> 8) & 0x7];
			int8_t UnitHours = bcd[(TC32_63 >> 16) & 0xF];
			int8_t TensHours = bcd[(TC32_63 >> 24) & 0x3];

			Result.Frames = (UnitFrames - 0x30) + ((TensFrames - 0x30) * 10);
			Result.Seconds = (UnitSeconds - 0x30) + ((TensSeconds - 0x30) * 10);
			Result.Minutes = (UnitMinutes - 0x30) + ((TensMinutes - 0x30) * 10);
			Result.Hours = (UnitHours - 0x30) + ((TensHours - 0x30) * 10);
			Result.bDropFrame = _bDropFrameFlag;
			return Result;
		}

		static void Convert2BCD(uint32_t inValue, uint32_t& outHigh, uint32_t& outLow)
		{
			outLow = inValue % 10;
			inValue /= 10;
			outHigh = inValue % 10;
		}

		NTV2_RP188 Helpers::ConvertTimecodeToRP188(const FTimecode& Timecode)
		{
			uint32_t HH, HL;
			Convert2BCD(Timecode.Hours, HH, HL);
			uint32_t MH, ML;
			Convert2BCD(Timecode.Minutes, MH, ML);
			uint32_t SH, SL;
			Convert2BCD(Timecode.Seconds, SH, SL);
			uint32_t FH, FL;
			Convert2BCD(Timecode.Frames, FH, FL);

			NTV2_RP188 Result;
//@TODO: Research. The value of 0xff070000 match the setup that we have at the office upstairs. May not always work.
//@TODO: In all the documentation it look like only the BIT(16) is used
			Result.fDBB = 0xff070000;
			Result.fHi = (HH << 24) + (HL << 16) + (MH << 8) + (ML);
			Result.fLo = (SH << 24) + (SL << 16) + (FH << 8) + (FL);

			if (Timecode.bDropFrame)
			{
				Result.fLo |= 0x01 << 10;	// Drop Frame:  timecode bit 10
			}

			return Result;
		}

		NTV2FrameBufferFormat Helpers::ConvertPixelFormatToFrameBufferFormat(EPixelFormat InPixelFormat)
		{
			switch (InPixelFormat)
			{
			case EPixelFormat::PF_8BIT_YCBCR:
				return NTV2_FBF_8BIT_YCBCR;
			case EPixelFormat::PF_10BIT_RGB:
				return NTV2_FBF_10BIT_RGB;
			case EPixelFormat::PF_10BIT_YCBCR:
				return NTV2_FBF_10BIT_YCBCR;
			case EPixelFormat::PF_8BIT_ARGB:
			default:
				return NTV2_FBF_ARGB;
			}
		}

		EPixelFormat Helpers::ConvertFrameBufferFormatToPixelFormat(NTV2FrameBufferFormat InPixelFormat)
		{
			switch (InPixelFormat)
			{
			case NTV2_FBF_8BIT_YCBCR:
				return EPixelFormat::PF_8BIT_YCBCR;
			case NTV2_FBF_10BIT_RGB:
				return EPixelFormat::PF_10BIT_RGB;
			case NTV2_FBF_10BIT_YCBCR:
				return EPixelFormat::PF_10BIT_YCBCR;
			case NTV2_FBF_ARGB:
			default:
				return EPixelFormat::PF_8BIT_ARGB;
			}
		}

		AJA_PixelFormat Helpers::ConvertToPixelFormat(EPixelFormat InPixelFormat)
		{
			switch (InPixelFormat)
			{
			case EPixelFormat::PF_8BIT_YCBCR:
				return AJA_PixelFormat_YCbCr8;
			case EPixelFormat::PF_10BIT_RGB:
				return AJA_PixelFormat_RGB10;
			case EPixelFormat::PF_10BIT_YCBCR:
				return AJA_PixelFormat_YCbCr10;
			case EPixelFormat::PF_8BIT_ARGB:
			default:
				return AJA_PixelFormat_ARGB8;
			}
		}

		NTV2HDRXferChars Helpers::ConvertToAjaHDRXferChars(EAjaHDRMetadataEOTF HDRXferChars)
		{
			switch(HDRXferChars)
			{
			case EAjaHDRMetadataEOTF::PQ:
				return NTV2_VPID_TC_PQ;
			case EAjaHDRMetadataEOTF::HLG:
				return NTV2_VPID_TC_HLG;
			case EAjaHDRMetadataEOTF::SDR:
				return NTV2_VPID_TC_SDR_TV;
			default:
				return NTV2_VPID_TC_Unspecified;
			}
		}
		
		EAjaHDRMetadataEOTF Helpers::ConvertFromAjaHDRXferChars(NTV2HDRXferChars HDRXferChars)
		{
			switch(HDRXferChars)
			{
			case NTV2_VPID_TC_PQ:
				return EAjaHDRMetadataEOTF::PQ;
			case NTV2_VPID_TC_HLG:
				return EAjaHDRMetadataEOTF::HLG;
			case NTV2_VPID_TC_SDR_TV:
				return EAjaHDRMetadataEOTF::SDR;
			default:
				return EAjaHDRMetadataEOTF::Unspecified;
			}
		}

		NTV2HDRColorimetry Helpers::ConvertToAjaHDRColorimetry(EAjaHDRMetadataGamut HDRColorimetry)
		{
			switch (HDRColorimetry)
			{
			case EAjaHDRMetadataGamut::Rec709:
				return NTV2_VPID_Color_Rec709;
			case EAjaHDRMetadataGamut::Rec2020:
				return NTV2_VPID_Color_UHDTV;
			default:
				return NTV2_VPID_Color_Unknown;
			}
		}

		EAjaHDRMetadataGamut Helpers::ConvertFromAjaHDRColorimetry(NTV2HDRColorimetry HDRColorimetry)
		{
			switch (HDRColorimetry)
			{
			case NTV2_VPID_Color_Rec709:
				return EAjaHDRMetadataGamut::Rec709;
			case NTV2_VPID_Color_UHDTV:
				return EAjaHDRMetadataGamut::Rec2020;
			default:
				return EAjaHDRMetadataGamut::Invalid;
			}
		}
		
		NTV2HDRLuminance Helpers::ConvertToAjaHDRLuminance(EAjaHDRMetadataLuminance HDRLuminance)
		{
			switch (HDRLuminance)
			{
			case EAjaHDRMetadataLuminance::ICtCp:
				return NTV2_VPID_Luminance_ICtCp;
			case EAjaHDRMetadataLuminance::YCbCr:
				return NTV2_VPID_Luminance_YCbCr;
			default:
				checkNoEntry();
				return NTV2_VPID_Luminance_YCbCr;
			}
		}
		
		EAjaHDRMetadataLuminance Helpers::ConvertFromAjaHDRLuminance(NTV2HDRLuminance HDRLuminance)
		{
			switch (HDRLuminance)
			{
			case NTV2_VPID_Luminance_ICtCp:
				return EAjaHDRMetadataLuminance::ICtCp;
			case NTV2_VPID_Luminance_YCbCr:
				return EAjaHDRMetadataLuminance::YCbCr;
			default:
				checkNoEntry();
				return EAjaHDRMetadataLuminance::YCbCr;
			}
		}

		NTV2FrameRate Helpers::ConvertToFrameRate(uint32_t InNumerator, uint32_t InDenominator)
		{
			if (InNumerator == 120    && InDenominator == 1)		{ return NTV2_FRAMERATE_12000; }
			if (InNumerator == 120000 && InDenominator == 1001)		{ return NTV2_FRAMERATE_11988; }
			if (InNumerator == 60     && InDenominator == 1)		{ return NTV2_FRAMERATE_6000; }
			if (InNumerator == 60000  && InDenominator == 1001)		{ return NTV2_FRAMERATE_5994; }
			if (InNumerator == 50     && InDenominator == 1)		{ return NTV2_FRAMERATE_5000; }
			if (InNumerator == 48     && InDenominator == 1)		{ return NTV2_FRAMERATE_4800; }
			if (InNumerator == 48000  && InDenominator == 1001)		{ return NTV2_FRAMERATE_4795; }
			if (InNumerator == 30     && InDenominator == 1)		{ return NTV2_FRAMERATE_3000; }
			if (InNumerator == 30000  && InDenominator == 1001)		{ return NTV2_FRAMERATE_2997; }
			if (InNumerator == 25     && InDenominator == 1)		{ return NTV2_FRAMERATE_2500; }
			if (InNumerator == 24     && InDenominator == 1)		{ return NTV2_FRAMERATE_2400; }
			if (InNumerator == 24000  && InDenominator == 1001)		{ return NTV2_FRAMERATE_2398; }
			if (InNumerator == 19     && InDenominator == 1)		{ return NTV2_FRAMERATE_1900; }
			if (InNumerator == 19000  && InDenominator == 1001)		{ return NTV2_FRAMERATE_1898; }
			if (InNumerator == 18     && InDenominator == 1)		{ return NTV2_FRAMERATE_1800; }
			if (InNumerator == 18000  && InDenominator == 1001)		{ return NTV2_FRAMERATE_1798; }
			if (InNumerator == 15     && InDenominator == 1)		{ return NTV2_FRAMERATE_1500; }
			if (InNumerator == 15000  && InDenominator == 1001)		{ return NTV2_FRAMERATE_1498; }

			return NTV2_FRAMERATE_UNKNOWN;
		}

		FFrameRate Helpers::ConvertToUnrealFramerate(NTV2VideoFormat InVideoFormat)
		{
			FFrameRate Result;
			switch (::GetNTV2FrameRateFromVideoFormat(InVideoFormat))
			{
			case NTV2_FRAMERATE_6000:	Result = FFrameRate(60,1);	break;
			case NTV2_FRAMERATE_5994:	Result = FFrameRate(60000,1001);	break;
			case NTV2_FRAMERATE_4800:	Result = FFrameRate(48,1);	break;
			case NTV2_FRAMERATE_4795:	Result = FFrameRate(48000, 1001);	break;
			case NTV2_FRAMERATE_3000:	Result = FFrameRate(30,1);	break;
			case NTV2_FRAMERATE_2997:	Result = FFrameRate(30000, 1001);	break;
			case NTV2_FRAMERATE_2500:	Result = FFrameRate(25, 1);	break;
			case NTV2_FRAMERATE_2400:	Result = FFrameRate(24, 1);	break;
			case NTV2_FRAMERATE_2398:	Result = FFrameRate(24000, 1001);	break;
			case NTV2_FRAMERATE_5000:	Result = FFrameRate(50, 1);	break;
			default:					break;
			}
			return Result;
		}

		TimecodeFormat Helpers::ConvertToTimecodeFormat(NTV2VideoFormat InVideoFormat)
		{
			TimecodeFormat Result = kTCFormatUnknown;
			switch (::GetNTV2FrameRateFromVideoFormat(InVideoFormat))
			{
			case NTV2_FRAMERATE_6000:	Result = kTCFormat60fps;	break;
			case NTV2_FRAMERATE_5994:	Result = kTCFormat60fpsDF;	break;
			case NTV2_FRAMERATE_4800:	Result = kTCFormat48fps;	break;  // kTCFormat48fpsDF doesn't exist
			case NTV2_FRAMERATE_4795:	Result = kTCFormat48fps;	break;
			case NTV2_FRAMERATE_3000:	Result = kTCFormat30fps;	break;
			case NTV2_FRAMERATE_2997:	Result = kTCFormat30fpsDF;	break;
			case NTV2_FRAMERATE_2500:	Result = kTCFormat25fps;	break;
			case NTV2_FRAMERATE_2400:	Result = kTCFormat24fps;	break;
			case NTV2_FRAMERATE_2398:	Result = kTCFormat24fps;	break;  // kTCFormat24fpsDF doesn't exist
			case NTV2_FRAMERATE_5000:	Result = kTCFormat50fps;	break;
			default:					break;
			}
			return Result;
		}

		//From ntv2fieldburn.cpp
		static ULWord GetRP188RegisterForInput(const NTV2Channel inInputSource)
		{
			switch (inInputSource)
			{
			case NTV2Channel::NTV2_CHANNEL1: return kRegRP188InOut1DBB;	//	reg 29
			case NTV2Channel::NTV2_CHANNEL2: return kRegRP188InOut2DBB;	//	reg 64
			case NTV2Channel::NTV2_CHANNEL3: return kRegRP188InOut3DBB;	//	reg 268
			case NTV2Channel::NTV2_CHANNEL4: return kRegRP188InOut4DBB;	//	reg 273
			case NTV2Channel::NTV2_CHANNEL5: return kRegRP188InOut5DBB;	//	reg 342
			case NTV2Channel::NTV2_CHANNEL6: return kRegRP188InOut6DBB;	//	reg 418
			case NTV2Channel::NTV2_CHANNEL7: return kRegRP188InOut7DBB;	//	reg 427
			case NTV2Channel::NTV2_CHANNEL8: return kRegRP188InOut8DBB;	//	reg 436
			default: return 0;
			}
		}

		bool Helpers::GetTimecode(CNTV2Card* InCard, NTV2Channel InChannel, const NTV2VideoFormat InVideoFormat, uint32_t InFrameIndex, ETimecodeFormat InTimecodeFormat, bool bInLogError, FTimecode& OutTimecode)
		{
			assert(InCard);

			ULWord TimecodePresent;
			AJA_CHECK(InCard->ReadRegister(GetRP188RegisterForInput(InChannel), TimecodePresent));
			if (!Helpers::IsDesiredTimecodePresent(InCard, InChannel, InTimecodeFormat, TimecodePresent, bInLogError))
			{
				return false;
			}
			
			NTV2_RP188 TimecodeValue;
			AJA_CHECK(InCard->GetRP188Data(InChannel, TimecodeValue));

			OutTimecode = Helpers::ConvertTimecodeFromRP188(TimecodeValue, InVideoFormat);
			return true;
		}

		bool Helpers::GetTimecode(CNTV2Card* InCard, EAnalogLTCSource AnalogLTCInput, const NTV2VideoFormat InVideoFormat, bool bInLogError, FTimecode& OutTimecode)
		{
			bool bPresent = true;
			InCard->GetLTCInputPresent(bPresent, (UWord)AnalogLTCInput);

			if (!bPresent)
			{
				if (bInLogError)
				{
					UE_LOGF(LogAjaCore, Warning,  "AJA: The timecode type is not present for LTC Input '%d' on device '%s'.\n", uint32_t(AnalogLTCInput)+1, InCard->GetDisplayName().c_str());
				}
				return false;
			}

			RP188_STRUCT TimecodeValue;
			if (!InCard->ReadAnalogLTCInput((UWord)AnalogLTCInput, TimecodeValue))
			{
				if (bInLogError)
				{
					UE_LOGF(LogAjaCore, Warning,  "AJA: The timecode type is not present for LTC Input '%d' on device '%s'.\n", uint32_t(AnalogLTCInput) + 1, InCard->GetDisplayName().c_str());
				}
				return false;
			}

			OutTimecode = Helpers::ConvertTimecodeFromRP188(TimecodeValue, InVideoFormat);
			return true;
		}

		ETimecodeFormat Helpers::GetTimecodeFormat(CNTV2Card* InCard, NTV2Channel InChannel)
		{
			ULWord TimecodePresent;
			AJA_CHECK(InCard->ReadRegister(GetRP188RegisterForInput(InChannel), TimecodePresent));

			// 0 LTC detected
			// 1 VITC1 detected
			// 2 VITC2 detected
			// 16 Timecode present
			// 17 Selected one is present
			// 18 LTC is present
			// 19 VITC1 is present
			// 0xFFFFFFFF means an invalid register read

			if (TimecodePresent == 0xFFFFFFFF || (TimecodePresent & BIT_16) == 0)
			{
				return ETimecodeFormat::TCF_None;
			}

			if (((TimecodePresent & BIT_1) != 0) || ((TimecodePresent & BIT_19) != 0))
			{
				return ETimecodeFormat::TCF_VITC1;
			}
			return ETimecodeFormat::TCF_LTC;
		}

		bool Helpers::TryVideoFormatIndexToNTV2VideoFormat(FAJAVideoFormat InVideoFormatIndex, NTV2VideoFormat& OutFoundVideoFormat)
		{
			OutFoundVideoFormat = static_cast<NTV2VideoFormat>(InVideoFormatIndex);
			return NTV2_IS_VALID_VIDEO_FORMAT(OutFoundVideoFormat);
		}

		std::optional<NTV2VideoFormat> Helpers::GetInputVideoFormat(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel, NTV2InputSource InInputSource, NTV2VideoFormat InExpectedVideoFormat, bool bSetSDIConversion, std::string& OutFailedReason)
		{
			std::optional<NTV2VideoFormat> OptionalFormat;
			NTV2VideoFormat FoundVideoFormat;

			assert(InCard);
			const bool ForRetailDisplay = true;

			bool bIsProgressive = ::IsProgressivePicture(InExpectedVideoFormat);
			FoundVideoFormat = InCard->GetInputVideoFormat(InInputSource, bIsProgressive);

			if (FoundVideoFormat == NTV2_FORMAT_UNKNOWN)
			{
				OutFailedReason = "Unknown video format.";
				return OptionalFormat;
			}

			// Convert the signal wire format to a 4k format
			const bool bQuad = (InTransportType == ETransportType::TT_SdiQuadSQ || InTransportType == ETransportType::TT_SdiQuadTSI) && NTV2_IS_QUAD_FRAME_FORMAT(InExpectedVideoFormat);
			if (bQuad)
			{
				FoundVideoFormat = ::GetQuadSizedVideoFormat(FoundVideoFormat);
			}

			const bool b372DualLink = InTransportType == ETransportType::TT_SdiDual && NTV2_IS_372_DUALLINK_FORMAT(InExpectedVideoFormat);
			if (b372DualLink)
			{
				FoundVideoFormat = Get372Format(FoundVideoFormat);
			}

			//Single link SDI 4k TSI is used when dealing with Kona 5 and Corvid 44 Retail (4k) firmware. It can't do 12g routing
			//When video feed is 4k, it is detected as 3840x2160 format but the card won't accept it
			//We need to convert it to the 1080 (3g) format that will be routed on the card
			//This will convert the native 4k format to 4x1080 one
			const bool bIsSingleTSI = InTransportType == ETransportType::TT_SdiSingle4kTSI;
			if (bIsSingleTSI)
			{
				FoundVideoFormat = GetSDI4kTSIFormat(FoundVideoFormat);
			}

			if (FoundVideoFormat == NTV2_FORMAT_UNKNOWN)
			{
				OutFailedReason = "Unknown video format after Transport Type.";
				return OptionalFormat;
			}

			if (InTransportType != ETransportType::TT_SdiDual && Helpers::IsSdiTransport(InTransportType)) // The card expect a B signal for SMTPE372
			{
				if (bSetSDIConversion)
				{
					SetSDIInLevelBtoLevelAConversion(InCard, InTransportType, InChannel, FoundVideoFormat, FoundVideoFormat);
				}
				else
				{
					//Flag that effects the on wire encoding of the SDI frame. 
					bool bIs3Gb = false;
					InCard->GetSDIInput3GbPresent(bIs3Gb, InChannel);
					if (bIs3Gb)
					{
						FoundVideoFormat = GetLevelA(FoundVideoFormat);
					}
				}
			}
			OptionalFormat = FoundVideoFormat;
			return OptionalFormat;
		}

		bool Helpers::CompareFormats(NTV2VideoFormat LHS, NTV2VideoFormat& RHS, std::string& OutFailedReason)
		{
			const bool ForRetailDisplay = true;
			const NTV2FrameRate LHSFrameRate = GetNTV2FrameRateFromVideoFormat(LHS);
			const NTV2FrameRate RHSFrameRate = GetNTV2FrameRateFromVideoFormat(LHS);
			if (LHSFrameRate != RHSFrameRate)
			{
				OutFailedReason = "The expected frame rate doesn't match the detected frame rate.\n  Expected: ";
				OutFailedReason.append(NTV2FrameRateToString(LHSFrameRate, ForRetailDisplay));
				OutFailedReason.append(" Detected: ");
				OutFailedReason.append(NTV2FrameRateToString(RHSFrameRate, ForRetailDisplay));
				return false;
			}

			const NTV2Standard LHSStandard = GetNTV2StandardFromVideoFormat(LHS);
			const NTV2Standard RHSStandard = GetNTV2StandardFromVideoFormat(RHS);
			if (LHSStandard != RHSStandard)
			{
				OutFailedReason = "The expected standard doesn't match the detected standard.\n  Expected: ";
				OutFailedReason.append(NTV2StandardToString(LHSStandard, ForRetailDisplay));
				OutFailedReason.append(" Detected: ");
				OutFailedReason.append(NTV2StandardToString(RHSStandard, ForRetailDisplay));
				return false;
			}
			return true;
		}

		bool Helpers::GetInputVideoFormat(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel, NTV2InputSource InInputSource, NTV2VideoFormat InExpectedVideoFormat, NTV2VideoFormat& OutFoundVideoFormat, bool bSetSDIConversion, std::string& OutFailedReason, bool bEnforceExpectedFormat)
		{
			std::optional<NTV2VideoFormat> VideoFormat = GetInputVideoFormat(InCard, InTransportType, InChannel, InInputSource, InExpectedVideoFormat, bSetSDIConversion, OutFailedReason);
			if (VideoFormat.has_value())
			{
				OutFoundVideoFormat = VideoFormat.value();
				if (bEnforceExpectedFormat)
				{
					return CompareFormats(InExpectedVideoFormat, OutFoundVideoFormat, OutFailedReason);
				}
				return true;
			}

			return false;
		}

		bool Helpers::GetInputHDRMetadata(CNTV2Card* InCard, NTV2Channel InChannel, FAjaHDROptions& OutHDRMetadata)
		{
			NTV2VPIDTransferCharacteristics EOTF = NTV2VPIDTransferCharacteristics::NTV2_VPID_TC_SDR_TV;
			NTV2VPIDColorimetry Colorimetry = NTV2VPIDColorimetry::NTV2_VPID_Color_Rec709;
			NTV2VPIDLuminance Luminance = NTV2VPIDLuminance::NTV2_VPID_Luminance_YCbCr;
			
			const bool bSuccess = InCard->GetVPIDTransferCharacteristics(EOTF, InChannel)
				&& InCard->GetVPIDColorimetry(Colorimetry, InChannel)
				&& InCard->GetVPIDLuminance(Luminance, InChannel);

			if (!bSuccess)
			{
				return false;
			}
			
			OutHDRMetadata.EOTF = ConvertFromAjaHDRXferChars(EOTF);
			OutHDRMetadata.Gamut = ConvertFromAjaHDRColorimetry(Colorimetry);
			OutHDRMetadata.Luminance = ConvertFromAjaHDRLuminance(Luminance);
			
			return true;
		}

		bool Helpers::SetSDIOutLevelAtoLevelBConversion(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel, NTV2VideoFormat InFormat, bool bValue)
		{
			const int32_t NumberOfLinkChannel = Helpers::GetNumberOfLinkChannel(InTransportType);

			bool bDoLevelConversion = ::NTV2DeviceCanDo3GLevelConversion(InCard->GetDeviceID()) && !::IsVideoFormatB(InFormat);
			if (bDoLevelConversion)
			{
				for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
				{
					InCard->SetSDIOutLevelAtoLevelBConversion(NTV2Channel(int32_t(InChannel) + ChannelIndex), bValue);
				}
			}

			return bDoLevelConversion;
		}
		
		void Helpers::SetSDIInLevelBtoLevelAConversion(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel, NTV2VideoFormat InFormat, NTV2VideoFormat& OutFormat)
		{
			const int32_t NumberOfLinkChannel = Helpers::GetNumberOfLinkChannel(InTransportType);

			//Flag that effects the on wire encoding of the SDI frame. 
			bool bIs3Gb = false;
			InCard->GetSDIInput3GbPresent(bIs3Gb, InChannel);
			if (bIs3Gb)
			{
				OutFormat = GetLevelA(InFormat);
				if (OutFormat != InFormat)
				{
					for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
					{
						InCard->SetSDIInLevelBtoLevelAConversion(NTV2Channel(int32_t(InChannel) + ChannelIndex), true);
					}
				}
				else
				{
					for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
					{
						InCard->SetSDIInLevelBtoLevelAConversion(NTV2Channel(int32_t(InChannel) + ChannelIndex), false);
					}
				}
			}
			else
			{
				for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
				{
					InCard->SetSDIInLevelBtoLevelAConversion(NTV2Channel(int32_t(InChannel) + ChannelIndex), false);
				}
			}
		}

		void Helpers::RouteSdiSignal(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel, NTV2VideoFormat InVideoFormat, NTV2FrameBufferFormat InPixelFormat, bool bIsInput, bool bIsInputColorRgb, bool bWillUseKey)
		{
			const bool bIsRGB = ::IsRGBFormat(InPixelFormat);
			const bool bIsSDI = Helpers::IsSdiTransport(InTransportType);
			const int32_t NumberOfLinkChannel = Helpers::GetNumberOfLinkChannel(InTransportType);
			const bool bUseCscRouting = bWillUseKey || bIsRGB != bIsInputColorRgb;

			AJA_CHECK(bIsSDI);
	
			if (bIsInput)
			{
				if (InTransportType == ETransportType::TT_SdiSingle || InTransportType == ETransportType::TT_SdiDual || InTransportType == ETransportType::TT_SdiQuadSQ)
				{
					for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
					{
						const NTV2Channel Channel = NTV2Channel(int32_t(InChannel) + ChannelIndex);

						const bool bIsKey = false;
						const bool bIs425 = false;
						const bool bIsDS2 = false;
						const bool bIsBInput = false;

						const NTV2OutputCrosspointID sdiInputWidgetOutputXpt = ::GetSDIInputOutputXptFromChannel(Channel, bIsDS2);
						const NTV2InputCrosspointID frameBufferInputXpt = ::GetFrameBufferInputXptFromChannel(Channel, bIsBInput);

						if (bUseCscRouting) // the input is YUV and we wants RGB
						{
							const NTV2InputCrosspointID cscWidgetVideoInputXpt = ::GetCSCInputXptFromChannel(Channel, bIsKey); // CSC widget's YUV input to SDI-In widget's output
							const NTV2OutputCrosspointID cscWidgetRGBOutputXpt = ::GetCSCOutputXptFromChannel(Channel, bIsKey, bIsRGB); // Frame store input to CSC widget's RGB output

							AJA_CHECK(InCard->Connect(frameBufferInputXpt, cscWidgetRGBOutputXpt));
							AJA_CHECK(InCard->Connect(cscWidgetVideoInputXpt, sdiInputWidgetOutputXpt));
						}
						else // the input is YUV and we wants YUV or the input is RGB and we want RGB
						{
							AJA_CHECK(InCard->Connect(frameBufferInputXpt, sdiInputWidgetOutputXpt));	// Frame store input to SDI-In widget's output
						}
					}
				}
				else if (InTransportType == ETransportType::TT_SdiQuadTSI)
				{
					bool bDo4TSI = true;

					for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
					{
						const NTV2Channel Channel = NTV2Channel(int32_t(InChannel) + ChannelIndex);
						const NTV2Channel ContextChannel = (NTV2Channel)(Channel / 2);
						const bool bIsBInput = ChannelIndex % 2 == 1;
						const bool bIsDS2 = bDo4TSI ? false : bIsBInput; //Use DS2 crosspoint
						const bool bIsKey = false;

						if (bUseCscRouting)
						{
							AJA_CHECK(InCard->Connect(::GetCSCInputXptFromChannel(Channel, bIsKey), ::GetSDIInputOutputXptFromChannel(bDo4TSI ? Channel : ContextChannel, bIsDS2)));
							AJA_CHECK(InCard->Connect(Get425MuxInput(Channel), ::GetCSCOutputXptFromChannel(Channel, bIsKey, bIsRGB)));
							AJA_CHECK(InCard->Connect(::GetFrameBufferInputXptFromChannel(ContextChannel, bIsBInput), Get425MuxOutput(Channel, bIsRGB)));
						}
						else
						{
							AJA_CHECK(InCard->Connect(Get425MuxInput(Channel), ::GetSDIInputOutputXptFromChannel(bDo4TSI ? Channel : ContextChannel, bIsDS2)));
							AJA_CHECK(InCard->Connect(::GetFrameBufferInputXptFromChannel(ContextChannel, bIsBInput), Get425MuxOutput(Channel, bIsRGB)));
						}
					}
				}
				else if (InTransportType == ETransportType::TT_SdiSingle4kTSI)
				{
					const int32_t ChannelValue = InChannel;
					const int32_t MuxOffset = (ChannelValue / 2) * 4; //4k SDI1 uses Mux 1-2 (0..3) 4k SDI3 uses Mux 3-4 (4..7)

					//For input, at least on Kona 5 Retail, there is no different between high and low framerate
					//It uses all input SDI (virtually for 3-4), one pin per input
					//TSI is using 1-2
					for (int32_t ChannelIndex = 0; ChannelIndex < 4; ++ChannelIndex)
					{
						const int32_t ChannelOffset = ChannelIndex / 2;
						const NTV2Channel FrameStoreChannel = NTV2Channel(ChannelValue + ChannelOffset);
						const NTV2Channel InputChannel = NTV2Channel(ChannelIndex);
						const NTV2Channel MuxChannel = NTV2Channel(ChannelIndex + MuxOffset);

						//CSC routing not supported for now. Can't do UHD on SDI 3 with CSC (RGB) output
						constexpr bool bIsDS2 = false;
						AJA_CHECK(InCard->Connect(Get425MuxInput(MuxChannel), ::GetSDIInputOutputXptFromChannel(InputChannel, bIsDS2)));

						const bool bIsBInput = ChannelIndex % 2 == 1;
						AJA_CHECK(InCard->Connect(::GetFrameBufferInputXptFromChannel(FrameStoreChannel, bIsBInput), Get425MuxOutput(MuxChannel, bIsRGB)));
					}
				}
				else
				{
					UE_LOGF(LogAjaCore, Warning,  "RouteSignal: This input routing is not supported. %s.\n", Helpers::TransportTypeToString(InTransportType));
				}
			}
			else //is output
			{
				//Start with output bandwith with default value. Configuration will enable the right one
				InCard->SetSDIOut6GEnable(InChannel, false);
				InCard->SetSDIOut12GEnable(InChannel, false);

				if (InTransportType == ETransportType::TT_SdiSingle || InTransportType == ETransportType::TT_SdiDual || InTransportType == ETransportType::TT_SdiQuadSQ)
				{
					for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
					{
						const NTV2Channel Channel = NTV2Channel(int32_t(InChannel) + ChannelIndex);

						const bool bIsDS2 = false;
						const bool bIs425 = false;
						const bool bIsKey = false;
						const NTV2OutputCrosspointID fsVidOutXpt = ::GetFrameBufferOutputXptFromChannel(Channel, bIsRGB, bIs425); //NTV2_XptFrameBuffer1YUV | NTV2_XptFrameBuffer1RGB | NTV2_XptFrameBuffer1_425YUV | NTV2_XptFrameBuffer1_425RGB
						const NTV2InputCrosspointID sdiOutputWidgetInputXpt = ::GetSDIOutputInputXpt(Channel, bIsDS2); //NTV2_XptSDIOut1Input | NTV2_XptSDIOut1InputDS2

						const NTV2InputCrosspointID cscVidInpXpt = ::GetCSCInputXptFromChannel(Channel, bIsKey); //NTV2_XptCSC1VidInput | NTV2_XptCSC1KeyInput
						const NTV2OutputCrosspointID cscVidOutXpt = ::GetCSCOutputXptFromChannel(Channel, bIsKey, bIsRGB); //NTV2_XptCSC1VidRGB | NTV2_XptCSC1VidYUV | NTV2_XptCSC1KeyYUV

						if (bUseCscRouting)
						{
							AJA_CHECK(InCard->Connect(cscVidInpXpt, fsVidOutXpt));
							AJA_CHECK(InCard->Connect(sdiOutputWidgetInputXpt, cscVidOutXpt));
						}
						else
						{
							AJA_CHECK(InCard->Connect(sdiOutputWidgetInputXpt, fsVidOutXpt));
						}
					}
				}
				else if (InTransportType == ETransportType::TT_SdiQuadTSI)
				{
					for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
					{
						const NTV2Channel Channel = NTV2Channel(int32_t(InChannel) + ChannelIndex);

						InCard->SetSDIOut12GEnable(Channel, false);

						const bool bIsDS2 = false;
						const bool bIsKey = false;

						if (bUseCscRouting)
						{
							AJA_CHECK(InCard->Connect(::GetCSCInputXptFromChannel(Channel, bIsKey), Get425MuxOutput(Channel, bIsRGB)));
							AJA_CHECK(InCard->Connect(::GetSDIOutputInputXpt(Channel, bIsDS2), ::GetCSCOutputXptFromChannel(Channel, bIsKey, bIsRGB)));
						}
						else
						{
							AJA_CHECK(InCard->Connect(::GetSDIOutputInputXpt(Channel, bIsDS2), Get425MuxOutput(Channel, bIsRGB)));
						}

						const bool bIs425 = ChannelIndex % 2 == 1;
						const NTV2Channel ContextChannel = (NTV2Channel)(Channel / 2);
						AJA_CHECK(InCard->Connect(Get425MuxInput(Channel), ::GetFrameBufferOutputXptFromChannel(ContextChannel, bIsRGB, bIs425)));
					}
				}
				else if (InTransportType == ETransportType::TT_SdiSingle4kTSI && InChannel < NTV2_CHANNEL5)
				{
					const int32_t ChannelValue = InChannel;
					const int32_t MuxOffset = (ChannelValue / 2) * 4; //4k SDI1 uses Mux 1-2 (0..3) 4k SDI3 uses Mux 3-4 (4..7)
					
					const bool bIsHighFrameRate = NTV2_IS_4K_HFR_VIDEO_FORMAT(InVideoFormat);
					if (bIsHighFrameRate)
					{
						//High frame rate connects all SDI outs (1 pin on each) in a virtual muxed way
						//SDI1 output will use FS1-2, SDI3 will use FS3-4
						//TSI routing is always used
						for (int32_t ChannelIndex = 0; ChannelIndex < 4; ++ChannelIndex)
						{
							const int32_t ChannelOffset = ChannelIndex / 2;
							const NTV2Channel FrameStoreChannel = NTV2Channel(ChannelValue + ChannelOffset);
							const NTV2Channel OutputChannel = NTV2Channel(ChannelIndex);
							const NTV2Channel MuxChannel = NTV2Channel(ChannelIndex + MuxOffset);

							//CSC routing not supported for now. Can't do UHD on SDI 3 with CSC (RGB) output
							constexpr bool bIsDS2 = false;
							InCard->Disconnect(::GetSDIOutputInputXpt(OutputChannel, bIsDS2));
							AJA_CHECK(InCard->Connect(::GetSDIOutputInputXpt(OutputChannel, bIsDS2), Get425MuxOutput(MuxChannel, bIsRGB)));

							const bool bIs425 = ChannelIndex % 2 == 1;
							InCard->Disconnect(Get425MuxInput(MuxChannel));
							AJA_CHECK(InCard->Connect(Get425MuxInput(MuxChannel), ::GetFrameBufferOutputXptFromChannel(FrameStoreChannel, bIsRGB, bIs425)));
						}

						InCard->SetSDIOut12GEnable(InChannel, true);
					}
					else
					{
						for (int32_t ChannelIndex = 0; ChannelIndex < 4; ++ChannelIndex)
						{
							const int32_t ChannelOffset = ChannelIndex / 2;
							const NTV2Channel OutputChannel = NTV2Channel(ChannelValue + ChannelOffset);
							const NTV2Channel MuxChannel = NTV2Channel(ChannelIndex + MuxOffset);

							//CSC routing not supported for now. Can't do UHD on SDI 3 with CSC (RGB) output
							const bool bIsDS2 = ChannelIndex % 2 == 1;
							InCard->Disconnect(::GetSDIOutputInputXpt(OutputChannel, bIsDS2));
							AJA_CHECK(InCard->Connect(::GetSDIOutputInputXpt(OutputChannel, bIsDS2), Get425MuxOutput(MuxChannel, bIsRGB)));

							const bool bIs425 = ChannelIndex % 2 == 1;
							InCard->Disconnect(Get425MuxInput(MuxChannel));
							AJA_CHECK(InCard->Connect(Get425MuxInput(MuxChannel), ::GetFrameBufferOutputXptFromChannel(OutputChannel, bIsRGB, bIs425)));
						}
						
						InCard->SetSDIOut6GEnable(InChannel, true);
					}
				}
				else
				{
					UE_LOGF(LogAjaCore, Warning,  "RouteSignal: This output routing is not supported. %s.\n", Helpers::TransportTypeToString(InTransportType));
				}
			}
		}

		void Helpers::RouteHdmiSignal(CNTV2Card* InCard, ETransportType InTransportType, NTV2InputSource InInputSource, NTV2Channel InChannel, NTV2FrameBufferFormat InPixelFormat, bool bIsInput)
		{
			const bool bIsRGB = ::IsRGBFormat(InPixelFormat);
			const bool bIsKey = false;
			const bool bIsSDI_DS2 = false;
			const int32_t NumberOfLinkChannel = Helpers::GetNumberOfLinkChannel(InTransportType);

			const bool bIsHDMI = Helpers::IsHdmiTransport(InTransportType);
			AJA_CHECK(bIsHDMI);

			NTV2LHIHDMIColorSpace InputColor = NTV2_LHIHDMIColorSpaceYCbCr;
			if (bIsInput)
			{
				AJA_CHECK(InCard->GetHDMIInputColor(InputColor, InChannel));
			}
			else
			{
				const NTV2HDMIColorSpace OutColorSpace = bIsRGB ? NTV2_HDMIColorSpaceRGB : NTV2_HDMIColorSpaceYCbCr;
				AJA_CHECK(InCard->SetHDMIOutColorSpace(OutColorSpace));
			}

			const bool bIsInputRGB = InputColor == NTV2_LHIHDMIColorSpaceRGB;
			const bool bUseCscRouting = bIsRGB != bIsInputRGB;

			if (bIsInput)
			{
				if (InTransportType == ETransportType::TT_Hdmi)
				{
					for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
					{
						const NTV2Channel Channel = NTV2Channel(int32_t(InChannel) + ChannelIndex);
						UWord HDMI_Quadrant = ChannelIndex;

						const NTV2OutputCrosspointID InputWidgetOutputXpt = ::GetInputSourceOutputXpt(InInputSource, bIsSDI_DS2, bIsInputRGB, HDMI_Quadrant);
						const NTV2InputCrosspointID FrameBufferInputXpt = ::GetFrameBufferInputXptFromChannel(Channel);
						const NTV2InputCrosspointID cscWidgetVideoInputXpt = ::GetCSCInputXptFromChannel(Channel);
						const NTV2OutputCrosspointID cscWidgetOutputXpt = ::GetCSCOutputXptFromChannel(Channel, bIsKey, bIsRGB);

						if (bUseCscRouting)
						{
							AJA_CHECK(InCard->Connect(FrameBufferInputXpt, cscWidgetOutputXpt)); // Frame store input to CSC widget's YUV output
							AJA_CHECK(InCard->Connect(cscWidgetVideoInputXpt, InputWidgetOutputXpt)); // CSC widget's RGB input to input widget's output
						}
						else
						{
							AJA_CHECK(InCard->Connect(FrameBufferInputXpt, InputWidgetOutputXpt)); // Frame store input to input widget's output
						}
					}
				}
				else if (InTransportType == ETransportType::TT_Hdmi4kTSI)
				{
					for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
					{
						const NTV2Channel Channel = NTV2Channel(int32_t(InChannel) + ChannelIndex);

						const UWord BaseQuadrantIndex = ChannelIndex % 2 == 0 ? 0 : 2; // First channel will handle Quad 1 and 2, Second channel will handle Quad 3 and 4

						for (UWord QuadrantIndex = BaseQuadrantIndex; QuadrantIndex < BaseQuadrantIndex + 2; QuadrantIndex++)
						{
							const bool bIsBInput = QuadrantIndex % 2 == 1;

							const NTV2OutputCrosspointID InputWidgetOutputXpt = ::GetInputSourceOutputXpt(InInputSource, bIsSDI_DS2, bIsInputRGB, QuadrantIndex);
							const NTV2InputCrosspointID FrameBufferInputXpt = ::GetFrameBufferInputXptFromChannel(Channel, bIsBInput);

							const int32_t BaseChannelForCSC = InChannel == NTV2_CHANNEL1 ? 0 : 4; // Use 4 last CSC when routing channel 3 in 4K mode.
							const NTV2Channel ChannelForCSC = (NTV2Channel)(BaseChannelForCSC + int32_t(QuadrantIndex));
							const NTV2InputCrosspointID cscWidgetVideoInputXpt = ::GetCSCInputXptFromChannel(ChannelForCSC);
							const NTV2OutputCrosspointID cscWidgetOutputXpt = ::GetCSCOutputXptFromChannel(ChannelForCSC, bIsKey, bIsRGB);

							if (bUseCscRouting)
							{
								AJA_CHECK(InCard->Connect(FrameBufferInputXpt, Get4KHDMI425MuxOutput(InInputSource, QuadrantIndex, bIsRGB))); // Frame store input to CSC widget's YUV output
								AJA_CHECK(InCard->Connect(Get4KHDMI425MuxInput(InInputSource, QuadrantIndex), cscWidgetOutputXpt));
								AJA_CHECK(InCard->Connect(cscWidgetVideoInputXpt, InputWidgetOutputXpt)); // CSC widget's RGB input to input widget's output
							}
							else
							{
								AJA_CHECK(InCard->Connect(FrameBufferInputXpt, Get4KHDMI425MuxOutput(InInputSource, QuadrantIndex, bIsRGB))); // Frame store input to MUX output
								AJA_CHECK(InCard->Connect(Get4KHDMI425MuxInput(InInputSource, QuadrantIndex), InputWidgetOutputXpt));
							}
						}
					}
				}
			}
			else
			{
				if (InTransportType == ETransportType::TT_Hdmi)
				{
					for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
					{
						const NTV2Channel Channel = NTV2Channel(int32_t(InChannel) + ChannelIndex);
						UWord HDMI_Quadrant = ChannelIndex;

						const bool bIs425 = ChannelIndex % 2 == 1;

						const NTV2OutputCrosspointID VidOutXpt = ::GetFrameBufferOutputXptFromChannel(Channel, bIsRGB, bIs425);
						const NTV2InputCrosspointID cscVidInpXpt = ::GetCSCInputXptFromChannel(Channel);
						const NTV2OutputCrosspointID cscVidOutXpt = ::GetCSCOutputXptFromChannel(Channel, bIsKey, bIsRGB);
						const NTV2InputCrosspointID OutputVidInpXpt = ::GetOutputDestInputXpt(NTV2_OUTPUTDESTINATION_HDMI, bIsSDI_DS2, HDMI_Quadrant);

						if (bUseCscRouting)
						{
							AJA_CHECK(InCard->Connect(cscVidInpXpt, VidOutXpt));
							AJA_CHECK(InCard->Connect(OutputVidInpXpt, cscVidOutXpt));
						}
						else
						{
							AJA_CHECK(InCard->Connect(OutputVidInpXpt, VidOutXpt));
						}
					}
				}
				else if (InTransportType == ETransportType::TT_Hdmi4kTSI)
				{
					for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
					{
						const NTV2Channel Channel = NTV2Channel(int32_t(InChannel) + ChannelIndex);
						UWord HDMI_Quadrant = ChannelIndex;

						const UWord BaseQuadrantIndex = ChannelIndex % 2 == 0 ? 0 : 2; // First channel will handle Quad 1 and 2, Second channel will handle Quad 3 and 4

						for (UWord QuadrantIndex = BaseQuadrantIndex; QuadrantIndex < BaseQuadrantIndex + 2; QuadrantIndex++)
						{
							const bool bIs425 = QuadrantIndex % 2 == 1;

							const int32_t BaseChannelForCSC = InChannel == NTV2_CHANNEL1 ? 0 : 4; // Use 4 last CSC when routing channel 3 in 4K mode.
							const NTV2Channel ChannelForCSC = (NTV2Channel)(BaseChannelForCSC + int32_t(QuadrantIndex));

							const NTV2OutputCrosspointID VidOutXpt = ::GetFrameBufferOutputXptFromChannel(Channel, bIsRGB, bIs425);
							const NTV2InputCrosspointID cscVidInpXpt = ::GetCSCInputXptFromChannel(ChannelForCSC);
							const NTV2OutputCrosspointID cscVidOutXpt = ::GetCSCOutputXptFromChannel(ChannelForCSC, bIsKey, bIsRGB);
							const NTV2InputCrosspointID OutputVidInpXpt = ::GetOutputDestInputXpt(NTV2_OUTPUTDESTINATION_HDMI, bIsSDI_DS2, QuadrantIndex);


							if (bUseCscRouting)
							{
								AJA_CHECK(InCard->Connect(cscVidInpXpt, VidOutXpt));
								AJA_CHECK(InCard->Connect(Get4KHDMI425MuxInput(InInputSource, QuadrantIndex), cscVidOutXpt));
								AJA_CHECK(InCard->Connect(OutputVidInpXpt, Get4KHDMI425MuxOutput(InInputSource, QuadrantIndex, bIsRGB)));

							}
							else
							{
								AJA_CHECK(InCard->Connect(OutputVidInpXpt, Get4KHDMI425MuxOutput(InInputSource, QuadrantIndex, bIsRGB)));
								AJA_CHECK(InCard->Connect(Get4KHDMI425MuxInput(InInputSource, QuadrantIndex), VidOutXpt));
							}
						}
					}
				}
			}
		}

		void Helpers::RouteKeySignal(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel, NTV2Channel InKeyChannel, NTV2FrameBufferFormat InPixelFormat, bool bIsInput)
		{
			const bool bIsRGB = ::IsRGBFormat(InPixelFormat);
			const bool bIsKey = true;
			const int32_t NumberOfLinkChannel = Helpers::GetNumberOfLinkChannel(InTransportType);

			if (Helpers::IsSdiTransport(InTransportType))
			{
				if (bIsInput)
				{
					for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
					{
						const NTV2Channel ChannelItt = NTV2Channel(int32_t(InChannel) + ChannelIndex);
						const NTV2Channel KeyChannelItt = NTV2Channel(int32_t(InKeyChannel) + ChannelIndex);

						const bool bIsDS2 = false;
						AJA_CHECK(InCard->Connect(::GetCSCInputXptFromChannel(InChannel, bIsKey), ::GetSDIInputOutputXptFromChannel(KeyChannelItt, bIsDS2)));
					}
				}
				else
				{
					for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
					{
						const NTV2Channel ChannelItt = NTV2Channel(int32_t(InChannel) + ChannelIndex);
						const NTV2Channel KeyChannelItt = NTV2Channel(int32_t(InKeyChannel) + ChannelIndex);

						const bool bIsDS2 = false;
						AJA_CHECK(InCard->Connect(::GetSDIOutputInputXpt(KeyChannelItt, bIsDS2), ::GetCSCOutputXptFromChannel(ChannelItt, bIsKey, bIsRGB)));
					}
				}
			}
			else
			{
				UE_LOGF(LogAjaCore, Warning,  "RouteKeySignal: Key routing is not supported. %s.\n", Helpers::TransportTypeToString(InTransportType));
			}
		}

		NTV2InputCrosspointID Helpers::Get425MuxInput(const NTV2Channel inChannel)
		{
			static const NTV2InputCrosspointID	g425MuxInput[] = {	NTV2_Xpt425Mux1AInput,		NTV2_Xpt425Mux1BInput,		NTV2_Xpt425Mux2AInput,		NTV2_Xpt425Mux2BInput,
																	NTV2_Xpt425Mux3AInput,		NTV2_Xpt425Mux3BInput,		NTV2_Xpt425Mux4AInput,		NTV2_Xpt425Mux4BInput };

			if (NTV2_IS_VALID_CHANNEL(inChannel))
			{
				return g425MuxInput[inChannel];
			}
			else
			{
				return NTV2_INPUT_CROSSPOINT_INVALID;
			}
		}

		NTV2OutputCrosspointID Helpers::Get425MuxOutput(const NTV2Channel inChannel, const bool inIsRGB)
		{
			static const NTV2OutputCrosspointID	g425MuxOutputYUV[] = {	NTV2_Xpt425Mux1AYUV,		NTV2_Xpt425Mux1BYUV,		NTV2_Xpt425Mux2AYUV,		NTV2_Xpt425Mux2BYUV,
																		NTV2_Xpt425Mux3AYUV,		NTV2_Xpt425Mux3BYUV,		NTV2_Xpt425Mux4AYUV,		NTV2_Xpt425Mux4BYUV };
			static const NTV2OutputCrosspointID	g425MuxOutputRGB[] = {	NTV2_Xpt425Mux1ARGB,		NTV2_Xpt425Mux1BRGB,		NTV2_Xpt425Mux2ARGB,		NTV2_Xpt425Mux2BRGB,
																		NTV2_Xpt425Mux3ARGB,		NTV2_Xpt425Mux3BRGB,		NTV2_Xpt425Mux4ARGB,		NTV2_Xpt425Mux4BRGB };

			if (NTV2_IS_VALID_CHANNEL(inChannel))
			{
				return inIsRGB ? g425MuxOutputRGB[inChannel] : g425MuxOutputYUV[inChannel];
			}
			else
			{
				return NTV2_OUTPUT_CROSSPOINT_INVALID;
			}
		}

		NTV2OutputCrosspointID Helpers::Get4KHDMI425MuxOutput(const NTV2InputSource InInputSource, const UWord inQuadrant, const bool inIsRGB)
		{
			static const NTV2OutputCrosspointID	g425MuxOutputYUV[] = { NTV2_Xpt425Mux1AYUV,		NTV2_Xpt425Mux1BYUV,		NTV2_Xpt425Mux2AYUV,		NTV2_Xpt425Mux2BYUV,
																		NTV2_Xpt425Mux3AYUV,		NTV2_Xpt425Mux3BYUV,		NTV2_Xpt425Mux4AYUV,		NTV2_Xpt425Mux4BYUV };
			static const NTV2OutputCrosspointID	g425MuxOutputRGB[] = { NTV2_Xpt425Mux1ARGB,		NTV2_Xpt425Mux1BRGB,		NTV2_Xpt425Mux2ARGB,		NTV2_Xpt425Mux2BRGB,
																		NTV2_Xpt425Mux3ARGB,		NTV2_Xpt425Mux3BRGB,		NTV2_Xpt425Mux4ARGB,		NTV2_Xpt425Mux4BRGB };

			constexpr int32_t NumElements = sizeof(g425MuxOutputYUV) / sizeof(NTV2OutputCrosspointID);

			if (NTV2_INPUT_SOURCE_IS_HDMI(InInputSource))
			{
				if (InInputSource != NTV2_INPUTSOURCE_HDMI1 && InInputSource != NTV2_INPUTSOURCE_HDMI2)
				{
					UE_LOGF(LogAjaCore, Warning,  "AJA: 4K HDMI is only supported in inputs 1 and 2.\n");
					return NTV2_OUTPUT_CROSSPOINT_INVALID;
				}
			}

			const size_t Index = InInputSource == NTV2_INPUTSOURCE_HDMI1 ? inQuadrant : inQuadrant + 4;
			return inIsRGB ? g425MuxOutputRGB[Index] : g425MuxOutputYUV[Index];
		}

		NTV2InputCrosspointID Helpers::Get4KHDMI425MuxInput(const NTV2InputSource InInputSource, const UWord inQuadrant)
		{
			static const NTV2InputCrosspointID	g425MuxInput[] = { NTV2_Xpt425Mux1AInput,		NTV2_Xpt425Mux1BInput,		NTV2_Xpt425Mux2AInput,		NTV2_Xpt425Mux2BInput,
																	NTV2_Xpt425Mux3AInput,		NTV2_Xpt425Mux3BInput,		NTV2_Xpt425Mux4AInput,		NTV2_Xpt425Mux4BInput };

			constexpr int32_t NumElements = sizeof(g425MuxInput) / sizeof(NTV2OutputCrosspointID);

			if (InInputSource != NTV2_INPUTSOURCE_HDMI1 && InInputSource != NTV2_INPUTSOURCE_HDMI2)
			{
				return NTV2_INPUT_CROSSPOINT_INVALID;
			}

			const size_t Index = InInputSource == NTV2_INPUTSOURCE_HDMI1 ? inQuadrant : inQuadrant + 4;

			if (Index >= NumElements)
			{
				return NTV2_INPUT_CROSSPOINT_INVALID;
			}

			return g425MuxInput[Index];
		}

		bool Helpers::ConvertTransportForDevice(CNTV2Card* InCard, uint32_t DeviceIndex, ETransportType& InOutTransportType, NTV2VideoFormat DesiredVideoFormat)
		{
			// For 4k formats, if the device doesn't support single 12g then the routing need to be in TSI
			if ((InOutTransportType == ETransportType::TT_SdiSingle || InOutTransportType == ETransportType::TT_Hdmi) && (::Is4KFormat(DesiredVideoFormat)))
			{
				NTV2DeviceID DeviceID = DEVICE_ID_NOTFOUND;
				if (InCard)
				{
					DeviceID = InCard->GetDeviceID();
				}
				else
				{
					CNTV2DeviceScanner Scanner;
					Scanner.ScanHardware();

					NTV2DeviceInfo DeviceInfo;
					if (!Scanner.GetDeviceInfo(DeviceIndex, DeviceInfo, false))
					{
						UE_LOGF(LogAjaCore, Error, "ConfigureVideo: Device not found.\n");
						return false;
					}
					DeviceID = DeviceInfo.deviceID;
				}

				if (!::NTV2DeviceCanDo12gRouting(DeviceID))
				{
					InOutTransportType = InOutTransportType == ETransportType::TT_SdiSingle ? ETransportType::TT_SdiSingle4kTSI : ETransportType::TT_Hdmi4kTSI;
				}
			}
			return true;
		}

		NTV2Channel Helpers::GetTransportTypeChannel(ETransportType InTransportType, NTV2Channel InChannel)
		{
			// We could manage 2 channels in TSI but still need 4 CSC.
			//But if the input are in RGB/YUV and we do not convert, then we may use 2 channels. But we need to make the difference between used CSC, FrameStore & Channel
			//ie. input HDMI, RGB, 4k: should reserve 2. Channel 1 may use FrameStore 1&2, TSI MUX 1&2. Channel 2 may use FrameStore 3&4, TSI MUX 3&4
			//ie. input HDMI, YUV, 4k: should reserve 4. Channel 1 may use FrameStore 1&2, TSI MUX 1&2, CSC 1&2&3&4. Channel 2 may use FrameStore 3&4, TSI MUX 3&4, CSC 5&6&7&8

			if (InTransportType == ETransportType::TT_SdiSingle || InTransportType == ETransportType::TT_Hdmi)
			{
				return InChannel;
			}
			else if (InTransportType == ETransportType::TT_SdiDual)
			{
				switch (InChannel)
				{
				case NTV2Channel::NTV2_CHANNEL1:
				case NTV2Channel::NTV2_CHANNEL2:
					return NTV2Channel::NTV2_CHANNEL1;
				case NTV2Channel::NTV2_CHANNEL3:
				case NTV2Channel::NTV2_CHANNEL4:
					return NTV2Channel::NTV2_CHANNEL3;
				case NTV2Channel::NTV2_CHANNEL5:
				case NTV2Channel::NTV2_CHANNEL6:
					return NTV2Channel::NTV2_CHANNEL5;
				case NTV2Channel::NTV2_CHANNEL7:
				case NTV2Channel::NTV2_CHANNEL8:
					return NTV2Channel::NTV2_CHANNEL7;
				}
				return NTV2Channel::NTV2_CHANNEL_INVALID;
			}
			else if (InTransportType == ETransportType::TT_SdiQuadSQ || InTransportType == ETransportType::TT_SdiQuadTSI)
			{
				switch (InChannel)
				{
				case NTV2Channel::NTV2_CHANNEL1:
				case NTV2Channel::NTV2_CHANNEL2:
				case NTV2Channel::NTV2_CHANNEL3:
				case NTV2Channel::NTV2_CHANNEL4:
					return NTV2Channel::NTV2_CHANNEL1;
				case NTV2Channel::NTV2_CHANNEL5:
				case NTV2Channel::NTV2_CHANNEL6:
				case NTV2Channel::NTV2_CHANNEL7:
				case NTV2Channel::NTV2_CHANNEL8:
					return NTV2Channel::NTV2_CHANNEL5;
				}
				return NTV2Channel::NTV2_CHANNEL_INVALID;
			}
			else if (InTransportType == ETransportType::TT_Hdmi4kTSI || InTransportType == ETransportType::TT_SdiSingle4kTSI)
			{
				switch(InChannel)
				{
				case NTV2Channel::NTV2_CHANNEL1:
				case NTV2Channel::NTV2_CHANNEL2:
					return NTV2Channel::NTV2_CHANNEL1;
				case NTV2Channel::NTV2_CHANNEL3:
				case NTV2Channel::NTV2_CHANNEL4:
					return NTV2Channel::NTV2_CHANNEL3;
				}
				return NTV2Channel::NTV2_CHANNEL_INVALID;
			}
			return InChannel;
		}

		int32_t Helpers::GetNumberOfLinkChannel(ETransportType InTransportType)
		{
			switch (InTransportType)
			{
			case AJA::ETransportType::TT_SdiSingle: return 1;
			case AJA::ETransportType::TT_SdiSingle4kTSI: return 2; // When we add support for RGB, we will need to add protection because we can't do in/out 4k in RGB because of CSC count
			case AJA::ETransportType::TT_SdiDual: return 2;
			case AJA::ETransportType::TT_SdiQuadSQ: return 4;
			case AJA::ETransportType::TT_SdiQuadTSI: return 4;
			case AJA::ETransportType::TT_Hdmi: return 1;
			case AJA::ETransportType::TT_Hdmi4kTSI: return 2;
			}
			return 0;
		}

		bool Helpers::IsTsiRouting(ETransportType InTransportType)
		{
			switch (InTransportType)
			{
			case AJA::ETransportType::TT_SdiSingle4kTSI:
			case AJA::ETransportType::TT_SdiQuadTSI:
			case AJA::ETransportType::TT_Hdmi4kTSI:
				return true;

			case AJA::ETransportType::TT_SdiSingle:
			case AJA::ETransportType::TT_SdiDual:
			case AJA::ETransportType::TT_SdiQuadSQ:
			case AJA::ETransportType::TT_Hdmi:
				return false;
			}
			return false;
		}

		bool Helpers::IsSdiTransport(ETransportType InTransportType)
		{
			switch (InTransportType)
			{
			case AJA::ETransportType::TT_SdiSingle:
			case AJA::ETransportType::TT_SdiSingle4kTSI:
			case AJA::ETransportType::TT_SdiDual:
			case AJA::ETransportType::TT_SdiQuadSQ:
			case AJA::ETransportType::TT_SdiQuadTSI:
				return true;
			case AJA::ETransportType::TT_Hdmi:
			case AJA::ETransportType::TT_Hdmi4kTSI:
				return false;
			}
			return false;
		}

		bool Helpers::IsHdmiTransport(ETransportType InTransportType)
		{
			switch (InTransportType)
			{
			case AJA::ETransportType::TT_Hdmi:
			case AJA::ETransportType::TT_Hdmi4kTSI:
				return true;
			case AJA::ETransportType::TT_SdiSingle:
			case AJA::ETransportType::TT_SdiSingle4kTSI:
			case AJA::ETransportType::TT_SdiDual:
			case AJA::ETransportType::TT_SdiQuadSQ:
			case AJA::ETransportType::TT_SdiQuadTSI:
				return false;
			}
			return false;
		}

		const char* Helpers::TransportTypeToString(ETransportType InTransportType)
		{
			switch (InTransportType)
			{
			case AJA::ETransportType::TT_SdiSingle: return "Single link";
			case AJA::ETransportType::TT_SdiSingle4kTSI: return "Single link (TSI)";
			case AJA::ETransportType::TT_SdiDual: return "Dual link";
			case AJA::ETransportType::TT_SdiQuadSQ: return "Quad link (SQ)";
			case AJA::ETransportType::TT_SdiQuadTSI: return "Quad link (TSI)";
			case AJA::ETransportType::TT_Hdmi: return "HDMI";
			case AJA::ETransportType::TT_Hdmi4kTSI: return "HDMI (TSI)";
			}
			return "<Invalid>";
		}

		const char* Helpers::ReferenceTypeToString(EAJAReferenceType InReferenceType)
		{
			switch (InReferenceType)
			{
			case EAJAReferenceType::EAJA_REFERENCETYPE_EXTERNAL: return "External";
			case EAJAReferenceType::EAJA_REFERENCETYPE_FREERUN: return "Free Run";
			case EAJAReferenceType::EAJA_REFERENCETYPE_INPUT: return "Input";
			}
			return "<Invalid>";
		}

		NTV2VideoFormat Helpers::Get372Format(NTV2VideoFormat InFormat)
		{
			switch (InFormat)
			{
			case NTV2VideoFormat::NTV2_FORMAT_1080psf_2500_2: return NTV2VideoFormat::NTV2_FORMAT_1080p_5000_B;
			case NTV2VideoFormat::NTV2_FORMAT_1080psf_2997_2: return NTV2VideoFormat::NTV2_FORMAT_1080p_5994_B;
			case NTV2VideoFormat::NTV2_FORMAT_1080psf_3000_2: return NTV2VideoFormat::NTV2_FORMAT_1080p_6000_B;
			case NTV2VideoFormat::NTV2_FORMAT_1080psf_2K_2398: return NTV2VideoFormat::NTV2_FORMAT_1080p_2K_4795_B;
			case NTV2VideoFormat::NTV2_FORMAT_1080psf_2K_2400: return NTV2VideoFormat::NTV2_FORMAT_1080p_2K_4800_B;
			case NTV2VideoFormat::NTV2_FORMAT_1080psf_2K_2500: return NTV2VideoFormat::NTV2_FORMAT_1080p_2K_5000_B;
			//case NTV2VideoFormat::NTV2_FORMAT_1080psf_2K_2997 return NTV2VideoFormat::NTV2_FORMAT_1080p_2K_5994_B;
			//case NTV2VideoFormat::NTV2_FORMAT_1080psf_2K_3000: return NTV2VideoFormat::NTV2_FORMAT_1080p_2K_6000_B;
			}
			return NTV2VideoFormat::NTV2_FORMAT_UNKNOWN;
		}

		NTV2VideoFormat Helpers::GetLevelA(NTV2VideoFormat InFormat)
		{
			switch (InFormat)
			{
			case NTV2VideoFormat::NTV2_FORMAT_1080p_5000_B: return NTV2VideoFormat::NTV2_FORMAT_1080p_5000_A;
			case NTV2VideoFormat::NTV2_FORMAT_1080p_5994_B: return NTV2VideoFormat::NTV2_FORMAT_1080p_5994_A;
			case NTV2VideoFormat::NTV2_FORMAT_1080p_6000_B: return NTV2VideoFormat::NTV2_FORMAT_1080p_6000_A;
			case NTV2VideoFormat::NTV2_FORMAT_1080p_2K_4795_B: return NTV2VideoFormat::NTV2_FORMAT_1080p_2K_4795_A;
			case NTV2VideoFormat::NTV2_FORMAT_1080p_2K_4800_B: return NTV2VideoFormat::NTV2_FORMAT_1080p_2K_4800_A;
			case NTV2VideoFormat::NTV2_FORMAT_1080p_2K_5000_B: return NTV2VideoFormat::NTV2_FORMAT_1080p_2K_5000_A;
			case NTV2VideoFormat::NTV2_FORMAT_1080p_2K_5994_B: return NTV2VideoFormat::NTV2_FORMAT_1080p_2K_5994_A;
			case NTV2VideoFormat::NTV2_FORMAT_1080p_2K_6000_B: return NTV2VideoFormat::NTV2_FORMAT_1080p_2K_6000_A;
			}
			return InFormat;
		}

		NTV2VideoFormat Helpers::GetSDI4kTSIFormat(NTV2VideoFormat InFormat)
		{
			switch (InFormat)
			{
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160p_2398: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080p_2398;
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160p_2400: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080p_2400;
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160p_2500: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080p_2500;
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160p_2997: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080p_2997;
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160p_3000: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080p_3000;
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160p_5000: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080p_5000;
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160p_5994: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080p_5994;
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160p_6000: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080p_6000;
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160psf_2398: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080psf_2398;
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160psf_2400: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080psf_2400;
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160psf_2500: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080psf_2500;
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160psf_2997: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080psf_2997;
			case NTV2VideoFormat::NTV2_FORMAT_3840x2160psf_3000: return NTV2VideoFormat::NTV2_FORMAT_4x1920x1080psf_3000;
			}
			return InFormat;
		}

		bool Helpers::IsCurrentInputField(CNTV2Card* InCard, NTV2Channel InChannel, NTV2FieldID InFieldId)
		{
			assert(InCard);

			ULWord RegisterNumber = kRegStatus;
			ULWord BitShift = 0;

			// from bool CNTV2Card::WaitForInputFieldID (const NTV2FieldID inFieldID, const NTV2Channel channel)
			static ULWord regNum[] = { kRegStatus, kRegStatus, kRegStatus2, kRegStatus2, kRegStatus2, kRegStatus2, kRegStatus2, kRegStatus2, 0 };
			static ULWord bitShift[] = { 21, 19, 21, 19, 17, 15, 13, 3, 0 };

			RegisterNumber = regNum[InChannel];
			BitShift = bitShift[InChannel];

			//	See if the field ID of the last input vertical interrupt is the one of interest...
			ULWord StatusValue(0);
			InCard->ReadRegister(RegisterNumber, StatusValue);
			NTV2FieldID	CurrentFieldID = (static_cast <NTV2FieldID>((StatusValue >> BitShift) & 0x1));
			return (CurrentFieldID == InFieldId);
		}

		bool Helpers::IsCurrentOutputField(CNTV2Card* InCard, NTV2Channel InChannel, NTV2FieldID InFieldId)
		{
			assert(InCard);

			ULWord RegisterNumber = kRegStatus;
			ULWord BitShift = 0;

			// from bool CNTV2Card::WaitForOutputFieldID(const NTV2FieldID inFieldID, const NTV2Channel channel)
			static ULWord regNum[] = { kRegStatus, kRegStatus,  kRegStatus, kRegStatus, kRegStatus2, kRegStatus2, kRegStatus2, kRegStatus2, 0 };
			static ULWord bitShift[] = { 23, 5, 3, 1, 9, 7, 5, 3, 0 };

			RegisterNumber = regNum[InChannel];
			BitShift = bitShift[InChannel];

			//	See if the field ID of the last input vertical interrupt is the one of interest...
			ULWord StatusValue(0);
			InCard->ReadRegister(RegisterNumber, StatusValue);
			NTV2FieldID	CurrentFieldID = (static_cast <NTV2FieldID>((StatusValue >> BitShift) & 0x1));
			return (CurrentFieldID == InFieldId);
		}

		AJA::FTimecode Helpers::AdjustTimecodeForUE(CNTV2Card* InCard, NTV2Channel InChannel, NTV2VideoFormat InVideoFormat, const FTimecode& InTimecode, const FTimecode& InPreviousTimecode, ULWord& InOutPreviousVerticalInterruptCount)
		{
			assert(InCard);
			
			FTimecode AdjustedTimecode = InTimecode;

			//No need to make adjustments for frame rate under 30fps
			const TimecodeFormat TcFormat = ConvertToTimecodeFormat(InVideoFormat);
			const bool bIsGreaterThan30 = (TcFormat == kTCFormat60fps || TcFormat == kTCFormat60fpsDF || TcFormat == kTCFormat48fps || TcFormat == kTCFormat50fps);
			const bool bIsInterlaced = !::IsProgressivePicture(InVideoFormat);

			if (bIsGreaterThan30 || bIsInterlaced)
			{
				ULWord NewInterruptCount = 0;
				InCard->GetInputVerticalInterruptCount(NewInterruptCount, InChannel);

				//For duplicate timecodes, bump the frame number when a new vertical interrupt event was detected.
				if (InPreviousTimecode == InTimecode)
				{
					if (NewInterruptCount != InOutPreviousVerticalInterruptCount)
					{
						++AdjustedTimecode.Frames;
					}
				}
				else
				{
					InOutPreviousVerticalInterruptCount = NewInterruptCount;
				}
			}
			
			return AdjustedTimecode;
		}

		AJA::FTimecode Helpers::AdjustTimecodeFromUE(NTV2VideoFormat InVideoFormat, const FTimecode& InTimecode)
		{
			FTimecode OutputTimecode = InTimecode;
			
			const TimecodeFormat TcFormat = ConvertToTimecodeFormat(InVideoFormat);
			const bool bIsGreaterThan30 = (TcFormat == kTCFormat60fps || TcFormat == kTCFormat60fpsDF || TcFormat == kTCFormat48fps || TcFormat == kTCFormat50fps);
			const bool bIsInterlaced = !::IsProgressivePicture(InVideoFormat);

			if (bIsGreaterThan30 || bIsInterlaced)
			{
				OutputTimecode.Frames /= 2;
			}

			return OutputTimecode;
		}

		bool Helpers::IsDesiredTimecodePresent(CNTV2Card* InCard, NTV2Channel InChannel, ETimecodeFormat InDesiredFormat, const ULWord InDBBRegister, bool bInLogError)
		{
			// 0 LTC detected
			// 1 VITC1 detected
			// 2 VITC2 detected
			// 16 Timecode present
			// 17 Selected one is present
			// 18 LTC is present
			// 19 VITC1 is present
			// 0xFFFFFFFF means an invalid register read
			
			if (InDBBRegister == 0xFFFFFFFF || (InDBBRegister & BIT_16) == 0)
			{
				if (bInLogError)
				{
					UE_LOGF(LogAjaCore, Warning,  "AJA: There is no timecode present for channel '%d' on device '%s'.\n", uint32_t(InChannel) + 1, InCard->GetDisplayName().c_str());
				}
				return false;
			}

			const ULWord TimecodeTypeMask = BIT_0 | BIT_1;
			bool bIsPresent = false;
			switch (InDesiredFormat)
			{
				case ETimecodeFormat::TCF_LTC:
				{
					bIsPresent = (InDBBRegister & TimecodeTypeMask) == 0;
					if (!bIsPresent)
					{
						if (bInLogError)
						{
							UE_LOGF(LogAjaCore, Warning,  "AJA: The timecode type is not present for channel '%d' on device '%s'.\n", uint32_t(InChannel) + 1, InCard->GetDisplayName().c_str());
						}
					}
					break;
				}
				case ETimecodeFormat::TCF_VITC1:
				{
					bIsPresent = (InDBBRegister & TimecodeTypeMask) == 1 || (InDBBRegister & BIT_1) == 2;//VITC1 or VITC2
					if (!bIsPresent)
					{
						if (bInLogError)
						{
							UE_LOGF(LogAjaCore, Warning,  "AJA: The timecode type is not present for channel '%d' on device '%s'.\n", uint32_t(InChannel) + 1, InCard->GetDisplayName().c_str());
						}
					}
					break;
				}
			}

			return bIsPresent;
		}

		NTV2Channel Helpers::GetOverrideChannel(NTV2InputSource InInputSource, NTV2Channel InChannel, ETransportType InTransportType)
		{
			if (InInputSource == NTV2_INPUTSOURCE_HDMI2 && InTransportType == ETransportType::TT_Hdmi4kTSI)
			{
				return NTV2_CHANNEL3;
			}
			return InChannel;
		}
	}
}
