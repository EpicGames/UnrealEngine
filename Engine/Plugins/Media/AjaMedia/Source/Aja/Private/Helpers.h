// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AjaCoreModule.h"
#include "Misc/FrameRate.h"

#define MSWindows 1
#define AJA_WINDOWS 1
#define AJA_NO_AUTOIMPORT 1

THIRD_PARTY_INCLUDES_START

__pragma(warning(disable: 4263))  /* Member function does not override any base class virtual member function. */
__pragma(warning(disable: 4264))  /* No override available for virtual member function from base. */

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#pragma warning(push)
#pragma warning(disable : 4005)
#endif //PLATFORM_WINDOWS

#include "ntv2testpatterngen.h"
#include "ajabase/common/timecodeburn.h"
#include "ajabase/system/process.h"
#include "ajabase/system/atomic.h"
#include "ajabase/system/lock.h"
#include "ajabase/system/event.h"
#include "ajabase/system/memory.h"
#include "ajabase/system/thread.h"
#include "ajabase/system/systemtime.h"

#include "ajantv2/includes/ntv2utils.h"
#include "ajantv2/includes/ntv2enums.h"
#include "ajantv2/includes/ntv2devicescanner.h"
#include "ajantv2/includes/ntv2card.h"
#include "ajantv2/includes/ntv2devicefeatures.h"
#include "ajantv2/includes/ntv2formatdescriptor.h"
#include "ajantv2/includes/ntv2rp188.h"

#if PLATFORM_WINDOWS
#pragma warning(pop)
#include "Windows/HideWindowsPlatformTypes.h"
#endif //PLATFORM_WINDOWS

THIRD_PARTY_INCLUDES_END

#include "AJALib.h"
#include <optional>

namespace AJA
{
	namespace Private
	{
		enum class ERP188Source
		{
			LTC = 0,
			VITC1 = 1,
			VITC2 = 2
		};

		enum class EAnalogLTCSource
		{
			LTC1 = 0,
			LTC2 //Max number of LTC from aja API
		};

#ifdef NDEBUG
//#define AJA_CHECK(FUNCTION) { if (!(FUNCTION)) { *reinterpret_cast<char*>(0) = 0; } }
#define AJA_CHECK ensure
#else
#define AJA_CHECK check
#endif

#define AJA_TEST_MEMORY_BUFFER  0
#define AJA_TEST_MEMORY_END_TAG 0x89

		struct Helpers
		{
			static bool GetTimecode(CNTV2Card* InCard, NTV2Channel InChannel, const NTV2VideoFormat InVideoFormat, uint32_t InFrameIndex, ETimecodeFormat InTimecodeFormat, bool bLogError, FTimecode& OutTimecode);
			static bool GetTimecode(CNTV2Card* InCard, EAnalogLTCSource AnalogLTCInput, const NTV2VideoFormat InVideoFormat, bool bInLogError, FTimecode& OutTimecode);
			static ETimecodeFormat GetTimecodeFormat(CNTV2Card* InCard, NTV2Channel InChannel);

			/* Output timecode will be usable directly by UE.
			 * If frame rate is greater than 30, frame number will be expanded to have a linear frame number up to frame rate -1. i.e. 0, 1, 2.., 59 
			 */
			static FTimecode ConvertTimecodeFromRP188(const RP188_STRUCT& InRP188, const NTV2VideoFormat InVideoFormat);
			static FTimecode ConvertTimecodeFromRP188(const NTV2_RP188& InRP188, const NTV2VideoFormat InVideoFormat);
			static NTV2_RP188 ConvertTimecodeToRP188(const FTimecode& Timecode);

			static NTV2FrameBufferFormat ConvertPixelFormatToFrameBufferFormat(EPixelFormat InPixelFormat);
			static EPixelFormat ConvertFrameBufferFormatToPixelFormat(NTV2FrameBufferFormat InPixelFormat);
			static AJA_PixelFormat ConvertToPixelFormat(EPixelFormat InPixelFormat);
			
			/** Convert HDR transfer characteristics from Unreal to Aja format. */
			static NTV2HDRXferChars ConvertToAjaHDRXferChars(EAjaHDRMetadataEOTF HDRXferChars);
			/** Convert HDR transfer characteristics from Aja to Unreal format. */
			static EAjaHDRMetadataEOTF ConvertFromAjaHDRXferChars(NTV2HDRXferChars XferChar);
			/** Convert HDR colorimetry from Unreal to Aja format. */
			static NTV2HDRColorimetry ConvertToAjaHDRColorimetry(EAjaHDRMetadataGamut Colorimetry);
			/** Convert HDR colorimetry from Aja to unreal format. */
			static EAjaHDRMetadataGamut ConvertFromAjaHDRColorimetry(NTV2HDRColorimetry);
			/** Convert HDR luminance from Unreal to Aja format. */
			static NTV2HDRLuminance ConvertToAjaHDRLuminance(EAjaHDRMetadataLuminance HDRLuminance);
			/** Convert HDR luminance from Aja to Unreal format. */
			static EAjaHDRMetadataLuminance ConvertFromAjaHDRLuminance(NTV2HDRLuminance AjaHDRLuminance);
			
			static NTV2FrameRate ConvertToFrameRate(uint32_t InNumerator, uint32_t InDenominator);
			static FFrameRate ConvertToUnrealFramerate(NTV2VideoFormat InVideoFormat);
			static TimecodeFormat ConvertToTimecodeFormat(NTV2VideoFormat InVideoFormat);

			static bool TryVideoFormatIndexToNTV2VideoFormat(FAJAVideoFormat InVideoFormatIndex, NTV2VideoFormat& OutFoundVideoFormat);
			static bool GetInputVideoFormat(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel, NTV2InputSource InInputSource, NTV2VideoFormat InExpectedVideoFormat, NTV2VideoFormat& OutFoundVideoFormat, bool bSetSDIConversion, std::string& OutFailedReason, bool bEnforceExpectedFormat = true);
			/** Fetch HDR Metadata from VPID registers. */
			static bool GetInputHDRMetadata(CNTV2Card* InCard, NTV2Channel InChannel, FAjaHDROptions& OutHDRMetadata);
			static std::optional<NTV2VideoFormat> GetInputVideoFormat(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel, NTV2InputSource InInputSource, NTV2VideoFormat InExpectedVideoFormat, bool bSetSDIConversion, std::string& OutFailedReason);
			static bool CompareFormats(NTV2VideoFormat LHS, NTV2VideoFormat& RHS, std::string& OutFailedReason);
			static void RouteSdiSignal(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel, NTV2VideoFormat InVideoFormat, NTV2FrameBufferFormat InPixelFormat, bool bIsInput, bool bIsInputColorRgb, bool bWillUseKey);
			static void RouteHdmiSignal(CNTV2Card* InCard, ETransportType InTransportType, NTV2InputSource InputSource, NTV2Channel InChannel, NTV2FrameBufferFormat InPixelFormat, bool bIsInput);
			static void RouteKeySignal(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel, NTV2Channel InKeyChannel, NTV2FrameBufferFormat InPixelFormat, bool bIsInput);
			static NTV2InputCrosspointID Get425MuxInput(const NTV2Channel inChannel);
			static NTV2OutputCrosspointID Get425MuxOutput(const NTV2Channel inChannel, const bool inIsRGB);
			static NTV2InputCrosspointID Get4KHDMI425MuxInput(const NTV2InputSource InInputSource, const UWord inQuadrant);
			static NTV2OutputCrosspointID Get4KHDMI425MuxOutput(const NTV2InputSource InInputSource, const UWord inQuadrant, const bool inIsRGB);

			static bool SetSDIOutLevelAtoLevelBConversion(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel, NTV2VideoFormat InFormat, bool bValue);
			static void SetSDIInLevelBtoLevelAConversion(CNTV2Card* InCard, ETransportType InTransportType, NTV2Channel InChannel, NTV2VideoFormat InFormat, NTV2VideoFormat& OutFormat);

			static bool ConvertTransportForDevice(CNTV2Card* InCard, uint32_t DeviceIndex, ETransportType& InOutTransportType, NTV2VideoFormat DesiredVideoFormat);

			static NTV2Channel GetTransportTypeChannel(ETransportType InTransportType, NTV2Channel InChannel);
			static int32_t GetNumberOfLinkChannel(ETransportType InTransportType);
			static bool IsTsiRouting(ETransportType InTransportType);
			static bool IsSdiTransport(ETransportType InTransportType);
			static bool IsHdmiTransport(ETransportType InTransportType);
			static const char* TransportTypeToString(ETransportType InTransportType);
			static const char* ReferenceTypeToString(EAJAReferenceType InReferenceType);

			static NTV2VideoFormat Get372Format(NTV2VideoFormat InFormat);
			static NTV2VideoFormat GetLevelA(NTV2VideoFormat InFormat);
			static NTV2VideoFormat GetSDI4kTSIFormat(NTV2VideoFormat InFormat);

			static bool IsCurrentInputField(CNTV2Card* InCard, NTV2Channel InChannel, NTV2FieldID InFieldId);
			static bool IsCurrentOutputField(CNTV2Card* InCard, NTV2Channel InChannel, NTV2FieldID InFieldId);

			static FTimecode AdjustTimecodeForUE(CNTV2Card* InCard, NTV2Channel InChannel, NTV2VideoFormat InVideoFormat, const FTimecode& InTimecode, const FTimecode& InPreviousTimecode, ULWord& InOutPreviousVerticalInterruptCount);
			static FTimecode AdjustTimecodeFromUE(NTV2VideoFormat InVideoFormat, const FTimecode& InTimecode);
			static bool IsDesiredTimecodePresent(CNTV2Card* InCard, NTV2Channel InChannel, ETimecodeFormat InDesiredFormat, const ULWord InDBBRegister, bool bInLogError);

			static NTV2Channel GetOverrideChannel(NTV2InputSource InInputSource, NTV2Channel InChannel, ETransportType InTransportType);
		};
	}
}
