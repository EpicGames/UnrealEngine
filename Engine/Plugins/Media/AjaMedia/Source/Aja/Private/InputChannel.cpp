// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputChannel.h"



namespace AJA
{
	namespace Private
	{
		/* InputChannel implementation
		*****************************************************************************/
		bool InputChannel::Initialize(const AJADeviceOptions& InDeviceOption, const AJAInputOutputChannelOptions& InOptions)
		{
			Uninitialize();
			ChannelThread = std::make_shared<InputChannelThread>(this, InDeviceOption, InOptions);

			bool bResult = ChannelThread->CanInitialize();
			if (bResult)
			{
				Device = DeviceCache::GetDevice(InDeviceOption);
				std::shared_ptr<IOChannelInitialize_DeviceCommand> SharedCommand(new IOChannelInitialize_DeviceCommand(Device, ChannelThread));
				InitializeCommand = SharedCommand;
				Device->AddCommand(std::static_pointer_cast<DeviceCommand>(SharedCommand));
			}

			return bResult;
		}

		void InputChannel::Uninitialize()
		{
			if (ChannelThread)
			{
				std::shared_ptr<IOChannelInitialize_DeviceCommand> SharedCommand = InitializeCommand.lock();
				if (SharedCommand && Device)
				{
					Device->RemoveCommand(std::static_pointer_cast<DeviceCommand>(SharedCommand));
				}

				ChannelThread->Uninitialize(); // delete is done in IOChannelUninitialize_DeviceCommand completed

				if (Device)
				{
					std::shared_ptr<IOChannelUninitialize_DeviceCommand> UninitializeCommand;
					UninitializeCommand.reset(new IOChannelUninitialize_DeviceCommand(Device, ChannelThread));
					Device->AddCommand(std::static_pointer_cast<DeviceCommand>(UninitializeCommand));
				}
			}

			InitializeCommand.reset();
			ChannelThread.reset();
			Device.reset();
		}

		AUTOCIRCULATE_STATUS InputChannel::GetCurrentAutoCirculateStatus() const
		{
			if (ChannelThread)
			{
				return ChannelThread->GetCurrentAutoCirculateStatus();
			}
			return AUTOCIRCULATE_STATUS();
		}

		const AJAInputOutputChannelOptions& InputChannel::GetOptions() const
		{
			return ChannelThread->GetOptions();
		}

		const AJADeviceOptions& InputChannel::GetDeviceOptions() const
		{
			return ChannelThread->DeviceOption;
		}

		/* InputChannelThread implementation
		*****************************************************************************/
		InputChannelThread::InputChannelThread(InputChannel* InOwner, const AJADeviceOptions& InDevice, const AJAInputOutputChannelOptions& InOptions)
			: Super(InDevice, InOptions)
			, AncBuffer(nullptr)
			, AncF2Buffer(nullptr)
			, AudioBuffer(nullptr)
			, VideoBuffer(nullptr)
			, PingPongDropCount(0)
			, Owner(InOwner)
		{}

		InputChannelThread::~InputChannelThread()
		{
			AJA_CHECK(AncBuffer == nullptr);
			AJA_CHECK(AncF2Buffer == nullptr);
			AJA_CHECK(AudioBuffer == nullptr);
			AJA_CHECK(VideoBuffer == nullptr);
		}

		void InputChannelThread::DeviceThread_Destroy(DeviceConnection::CommandList& InCommandList)
		{
			if (AncBuffer)
			{
				AJAMemory::FreeAligned(AncBuffer);
			}
			if (AncF2Buffer)
			{
				AJAMemory::FreeAligned(AncF2Buffer);
			}
			if (AudioBuffer)
			{
				AJAMemory::FreeAligned(AudioBuffer);
			}
			if (VideoBuffer)
			{
				AJAMemory::FreeAligned(VideoBuffer);
			}

			AncBuffer = nullptr;
			AncF2Buffer = nullptr;
			AudioBuffer = nullptr;
			VideoBuffer = nullptr;

			Super::DeviceThread_Destroy(InCommandList);
		}

		bool InputChannelThread::DeviceThread_ConfigureAnc(DeviceConnection::CommandList& InCommandList)
		{
			AJA_CHECK(AncBuffer == nullptr);
			bool bResult = Super::DeviceThread_ConfigureAnc(InCommandList);

			if (AncBufferSize > 0)
			{
				AJA_CHECK(UseAncillary());
#if AJA_TEST_MEMORY_BUFFER
				AncBuffer = (uint8_t*)AJAMemory::AllocateAligned(AncBufferSize + 1, AJA_PAGE_SIZE);
				AncBuffer[AncBufferSize] = AJA_TEST_MEMORY_END_TAG;
#else
				AncBuffer = (uint8_t*)AJAMemory::AllocateAligned(AncBufferSize, AJA_PAGE_SIZE);
#endif
			}

			if (AncF2BufferSize > 0)
			{
				AJA_CHECK(UseAncillaryField2());
#if AJA_TEST_MEMORY_BUFFER
				AncF2Buffer = (uint8_t*)AJAMemory::AllocateAligned(AncF2BufferSize + 1, AJA_PAGE_SIZE);
				AncF2Buffer[AncF2BufferSize] = AJA_TEST_MEMORY_END_TAG;
#else
				AncF2Buffer = (uint8_t*)AJAMemory::AllocateAligned(AncF2BufferSize, AJA_PAGE_SIZE);
#endif
			}

			return bResult;
		}

		bool InputChannelThread::DeviceThread_ConfigureAudio(DeviceConnection::CommandList& InCommandList)
		{
			AJA_CHECK(AudioBuffer == nullptr);
			bool bResult = Super::DeviceThread_ConfigureAudio(InCommandList);

			if (AudioBufferSize > 0)
			{
#if AJA_TEST_MEMORY_BUFFER
				AudioBuffer = (uint8_t*)AJAMemory::AllocateAligned(AudioBufferSize + 1, AJA_PAGE_SIZE);
				AudioBuffer[AudioBufferSize] = AJA_TEST_MEMORY_END_TAG;
#else
				AudioBuffer = (uint8_t*)AJAMemory::AllocateAligned(AudioBufferSize, AJA_PAGE_SIZE);
#endif
			}
			return bResult;
		}

		bool InputChannelThread::DeviceThread_ConfigureVideo(DeviceConnection::CommandList& InCommandList)
		{
			AJA_CHECK(VideoBuffer == nullptr);
			bool bResult = Super::DeviceThread_ConfigureVideo(InCommandList);

			if (VideoBufferSize > 0)
			{
				AJA_CHECK(UseVideo());
#if AJA_TEST_MEMORY_BUFFER
				VideoBuffer = (uint8_t*)AJAMemory::AllocateAligned(VideoBufferSize + 1, AJA_PAGE_SIZE);
				VideoBuffer[VideoBufferSize] = AJA_TEST_MEMORY_END_TAG;
#else
				VideoBuffer = (uint8_t*)AJAMemory::AllocateAligned(VideoBufferSize, AJA_PAGE_SIZE);
#endif
			}
			return bResult;
		}

		bool InputChannelThread::DeviceThread_ConfigureAutoCirculate(DeviceConnection::CommandList& InCommandList)
		{
			NTV2DeviceID DeviceId = GetDevice().GetDeviceID();
			if (!::NTV2DeviceCanDoCapture(DeviceId))
			{
				UE_LOGF(LogAjaCore, Error, "AutoCirculate: The device '%s' couldn't not capture.", GetDevice().GetDisplayName().c_str());
				return false;
			}

			const int32_t NumberOfLinkChannel = Helpers::GetNumberOfLinkChannel(GetOptions().TransportType);
			for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
			{
				GetDevice().AutoCirculateStop(NTV2Channel(int32_t(Channel) + ChannelIndex));
			}

			{
				// NB. We are already locked
				ULWord OptionFlags = (UseTimecode() ? AUTOCIRCULATE_WITH_RP188 : 0);
				OptionFlags |= (UseAncillary() || UseAncillaryField2() ? AUTOCIRCULATE_WITH_ANC : 0);
				UByte FrameCount = 0;
				UByte FirstFrame = BaseFrameIndex;
				UByte EndFrame = FirstFrame + DeviceConnection::NumberOfFrameForAutoCirculate - 1;

				AJA_CHECK(GetDevice().AutoCirculateInitForInput(Channel, FrameCount, AudioSystem, OptionFlags, 1, FirstFrame, EndFrame));
			}
			AJA_CHECK(GetDevice().AutoCirculateStart(Channel));

			uint32_t Counter = 0;
			bool bRunning = true;
			while (bRunning && !bStopRequested)
			{
				AUTOCIRCULATE_STATUS ChannelStatus;
				bRunning = GetDevice().AutoCirculateGetStatus(Channel, ChannelStatus);
				if (!bRunning)
				{
					UE_LOGF(LogAjaCore, Error, "AutoCirculate: Can't get the status for channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
					bRunning = false;
				}

				if (ChannelStatus.IsRunning())
				{
					return true;
				}

				const DWORD SleepMilli = 10;
				const DWORD TimeoutMilli = 2000;
				if (Counter * SleepMilli > TimeoutMilli)
				{
					UE_LOGF(LogAjaCore, Error, "AutoCirculate: Can't get the Channel running for channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
					bRunning = false;
				}

				++Counter;
				::Sleep(SleepMilli);
			}

			if (!bRunning)
			{
				AJA_CHECK(GetDevice().AutoCirculateStop(Channel));
			}

			return bRunning;
		}

		void InputChannelThread::Thread_AutoCirculateLoop()
		{
			FTimecode PreviousTimecode;
			ULWord PreviousVerticalInterruptCount = 0;
			AUTOCIRCULATE_TRANSFER Transfer;
			bool bRunning = true;
			// To prevent spamming the logs, we will only print this once.
			bool bLogTimecodeError = true;

			while (bRunning && !bStopRequested)
			{
				AUTOCIRCULATE_STATUS ChannelStatus;
				bRunning = GetDevice().AutoCirculateGetStatus(Channel, ChannelStatus);
				if (!bRunning)
				{
					UE_LOGF(LogAjaCore, Error, "AutoCirculate: Can't get the status for channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
					break;
				}

				if (ChannelStatus.IsRunning() && ChannelStatus.HasAvailableInputFrame())
				{
					// Request buffers
					AJARequestInputBufferData RequestInputData;
					RequestInputData.bIsProgressivePicture = ::IsProgressivePicture(VideoFormat);
					if (AncBuffer || AncF2Buffer)
					{
						RequestInputData.AncBufferSize = AncBufferSize;
						RequestInputData.AncF2BufferSize = AncF2BufferSize;
					}
					if (AudioBuffer)
					{
						RequestInputData.AudioBufferSize = AudioBufferSize;
					}
					if (VideoBuffer && ::IsProgressivePicture(VideoFormat))
					{
						RequestInputData.VideoBufferSize = VideoBufferSize;
					}

					AJARequestedInputBufferData RequestedBuffer;
					{
						AJAAutoLock AutoLock(&Lock);
						if (GetOptions().CallbackInterface)
						{
							bRunning = GetOptions().CallbackInterface->OnRequestInputBuffer(RequestInputData, RequestedBuffer);
						}
					}

					// Set the buffers for the transfer
					AJAAncillaryFrameData AncillaryData;
					if (AncBuffer || AncF2Buffer)
					{
						AJA_CHECK(UseAncillary() || UseAncillaryField2());
						AJA_CHECK(AncBufferSize > 0 || AncF2BufferSize > 0);
						AncillaryData.AncBuffer = RequestedBuffer.AncBuffer ? RequestedBuffer.AncBuffer : AncBuffer;
						AncillaryData.AncF2Buffer = RequestedBuffer.AncBuffer ? RequestedBuffer.AncF2Buffer : AncF2Buffer;
						Transfer.SetAncBuffers(reinterpret_cast<ULWord*>(AncillaryData.AncBuffer), AncBufferSize, reinterpret_cast<ULWord*>(AncillaryData.AncF2Buffer), AncF2BufferSize);
					}


					AJAAudioFrameData AudioData;
					if (AudioBuffer)
					{
#if AJA_TEST_MEMORY_BUFFER
						AJA_CHECK(AudioBuffer[AudioBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif
						AJA_CHECK(NTV2_IS_VALID_AUDIO_SYSTEM(AudioSystem));
						AJA_CHECK(UseAudio());
						AJA_CHECK(AudioBufferSize > 0);
						AudioData.AudioBuffer = RequestedBuffer.AudioBuffer ? RequestedBuffer.AudioBuffer : AudioBuffer;
						Transfer.SetAudioBuffer(reinterpret_cast<ULWord*>(AudioData.AudioBuffer), AudioBufferSize);

#if AJA_TEST_MEMORY_BUFFER
						AJA_CHECK(AudioBuffer[AudioBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif
					}


					AJAVideoFrameData VideoData;
					VideoData.HDROptions = GetOptions().HDROptions;
					
					if (VideoBuffer)
					{
#if AJA_TEST_MEMORY_BUFFER
						AJA_CHECK(VideoBuffer[VideoBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif
						AJA_CHECK(UseVideo());
						AJA_CHECK(VideoBufferSize > 0);
						VideoData.VideoBuffer = RequestedBuffer.VideoBuffer ? RequestedBuffer.VideoBuffer : VideoBuffer;
						Transfer.SetVideoBuffer(reinterpret_cast<ULWord*>(VideoData.VideoBuffer), VideoBufferSize);

#if AJA_TEST_MEMORY_BUFFER
						AJA_CHECK(VideoBuffer[VideoBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif
					}

					bRunning = GetDevice().AutoCirculateTransfer(Channel, Transfer);
					if (!bRunning)
					{
						UE_LOGF(LogAjaCore, Error, "AutoCirculate: Can't transfer the buffer for channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
						break;
					}

					FTimecode Timecode;
					if (UseTimecode())
					{
						// read the timecode from the hardware
						NTV2_RP188 InputTimeCode;
						const bool bUseLTCTimecode = GetOptions().TimecodeFormat == ETimecodeFormat::TCF_LTC;
						Transfer.GetInputTimeCode(InputTimeCode, NTV2ChannelToTimecodeIndex(Channel, bUseLTCTimecode));

						if (Helpers::IsDesiredTimecodePresent(Device->GetCard(), Channel, GetOptions().TimecodeFormat, InputTimeCode.fDBB, bLogTimecodeError))
						{
							const FTimecode ConvertedTimecode = Helpers::ConvertTimecodeFromRP188(InputTimeCode, VideoFormat);
							Timecode = Helpers::AdjustTimecodeForUE(Device->GetCard(), Channel, VideoFormat, ConvertedTimecode, PreviousTimecode, PreviousVerticalInterruptCount);
							PreviousTimecode = ConvertedTimecode;
						}
						else
						{
							bLogTimecodeError = false;
						}
					}

					// Call the client
					if (!bStopRequested && GetOptions().CallbackInterface)
					{
						AJAInputFrameData InputFrameData;
						InputFrameData.FramesDropped = ChannelStatus.acFramesDropped;
						InputFrameData.Timecode = Timecode;

						if (AncillaryData.AncBuffer)
						{
							AncillaryData.AncBufferSize = Transfer.GetAncByteCount(false);
						}

						if (AncillaryData.AncF2Buffer)
						{
							AncillaryData.AncF2BufferSize = Transfer.GetAncByteCount(true);
						}

						if (AudioData.AudioBuffer)
						{
							NTV2FrameRate FrameRate = NTV2_FRAMERATE_INVALID;
							AJA_CHECK(GetDevice().GetFrameRate(FrameRate, Channel));

							NTV2AudioRate AudioRate = NTV2_AUDIO_RATE_INVALID;
							AJA_CHECK(GetDevice().GetAudioRate(AudioRate, AudioSystem));

							ULWord NumChannels;
							AJA_CHECK(GetDevice().GetNumberAudioChannels(NumChannels, AudioSystem));

							AudioData.AudioBufferSize = NTV2_IS_VALID_AUDIO_SYSTEM(AudioSystem) ? Transfer.GetCapturedAudioByteCount() : 0;
							AudioData.NumChannels = NumChannels;
							AudioData.AudioRate = AudioRate == NTV2_AUDIO_96K ? 96000 : 48000;
							AudioData.NumSamples = ::GetAudioSamplesPerFrame(FrameRate, AudioRate);
						}

						if (VideoData.VideoBuffer)
						{
							EVerifyFormatResult Result = VerifyFormat();
							if (Result == EVerifyFormatResult::Failure)
							{
								if (!GetOptions().bAutoDetectFormat)
								{
									bRunning = false;
									break;
								}
							}

							if (Result == EVerifyFormatResult::FormatChange)
							{
								break;
							}

							if (Result == EVerifyFormatResult::Success)
							{
								VideoData.VideoFormatIndex = VideoFormat;
								VideoData.VideoBufferSize = VideoBufferSize;
								VideoData.Width = FormatDescriptor.GetRasterWidth();
								VideoData.Height = FormatDescriptor.GetRasterHeight();
								VideoData.Stride = FormatDescriptor.GetBytesPerRow();
								VideoData.PixelFormat = Helpers::ConvertFrameBufferFormatToPixelFormat(FormatDescriptor.GetPixelFormat());

								VideoData.bIsProgressivePicture = ::IsProgressivePicture(VideoFormat);
								BurnTimecode(Timecode, VideoData.VideoBuffer);
							}
						}

						{
							AJAAutoLock AutoLock(&Lock);
							if (GetOptions().CallbackInterface)
							{
								bRunning = GetOptions().CallbackInterface->OnInputFrameReceived(InputFrameData, AncillaryData, AudioData, VideoData);
							}
						}

						Device->OnInputFrameReceived(Channel);
					}

					bool bWaitResult = Device->WaitForInputOrOutputInterrupt(Channel);
					if (!bWaitResult)
					{
						if (!Options.bAutoDetectFormat && Options.bStopOnTimeout)
						{
							AJA_CHECK(false);
						}
						else
						{
							UE_LOGF(LogAjaCore, Warning,  "AutoCirculate: The device '%s' missed an interrupt signal.", GetDevice().GetDisplayName().c_str());
						}
					}
				}
				else
				{
					EVerifyFormatResult Result = VerifyFormat();
					if (Result == EVerifyFormatResult::Failure)
					{
						if (!GetOptions().bAutoDetectFormat && GetOptions().bStopOnTimeout)
						{
							bRunning = false;
							break;
						}
					}

					if (Result == EVerifyFormatResult::FormatChange)
					{
						break;
					}

					::Sleep(0);
				}
			}

			GetDevice().AutoCirculateStop(Channel);

			if (!bStopRequested)
			{
				AJAAutoLock AutoLock(&Lock);
				if (GetOptions().CallbackInterface)
				{
					GetOptions().CallbackInterface->OnCompletion(bRunning);
				}
			}
		}

		// As reference: this code was inspirered by NTV2LLBurn::ProcessFrames(void)
		bool InputChannelThread::DeviceThread_ConfigurePingPong(DeviceConnection::CommandList& InCommandList)
		{
			NTV2DeviceID DeviceId = GetDevice().GetDeviceID();
			if (!::NTV2DeviceCanDoCapture(DeviceId))
			{
				UE_LOGF(LogAjaCore, Error, "PingPong: The device '%s' couldn't not capture.", GetDevice().GetDisplayName().c_str());
				return false;
			}

			if (NTV2_IS_VALID_AUDIO_SYSTEM(AudioSystem))
			{
				GetDevice().StopAudioInput(AudioSystem);
				GetDevice().SetAudioCaptureEnable(AudioSystem, true);
			}

			AJA_CHECK(Device->WaitForInputOrOutputInterrupt(Channel, 10));

			return true;
		}

		// As reference: this code was inspirered by NTV2LLBurn::ProcessFrames(void)
		void InputChannelThread::Thread_PingPongLoop()
		{
			FTimecode PreviousTimecode;
			ULWord PreviousVerticalInterruptCount = 0;
			uint32_t AudioReadOffset = 0;
			uint32_t AudioOutWrapAddress = 0;
			uint32_t AudioInLastAddress = 0;
			uint32_t AudioInWrapAddress = 0;
			uint32_t AudioBytesCaptured = 0;
			if (NTV2_IS_VALID_AUDIO_SYSTEM(AudioSystem))
			{
				AJA_CHECK(GetDevice().GetAudioReadOffset(AudioReadOffset, AudioSystem));
				AJA_CHECK(GetDevice().GetAudioWrapAddress(AudioOutWrapAddress, AudioSystem));
				AudioInLastAddress = AudioReadOffset;
				AudioInWrapAddress = AudioOutWrapAddress + AudioReadOffset;
			}

			//	Before the main loop starts, ping-pong the buffers so the hardware will use
			//	different buffers than the ones it was using while idling...
			uint32_t CurrentInFrame = 0;
			CurrentInFrame ^= 1;
			AJA_CHECK(GetDevice().SetInputFrame(Channel, BaseFrameIndex + CurrentInFrame));

			if (NTV2_IS_VALID_AUDIO_SYSTEM(AudioSystem))
			{
				GetDevice().StartAudioInput(AudioSystem);
			}

			bool bTimecodeIssue = false;
			bool bRunning = true;
			bool bNoFormatDetected = false;
			// To prevent spamming the logs, we will only print this once.
			bool bLogTimecodeError = true;

			while (bRunning && !bStopRequested)
			{
				//	Wait until the input has completed capturing a frame...
				bool bWaitResult = Device->WaitForInputOrOutputInterrupt(Channel);
				if (!bWaitResult)
				{
					if (!Options.bAutoDetectFormat && Options.bStopOnTimeout)
					{
						bRunning = false;
						AJA_CHECK(false);
					}
					else
					{
						UE_LOGF(LogAjaCore, Warning,  "PingPong: The device '%s' missed an interrupt signal.", GetDevice().GetDisplayName().c_str());
						::Sleep(0);
						continue;
					}
				}

				if (!bRunning)
				{
					UE_LOGF(LogAjaCore, Error, "PingPong: Can't wait for the input field for channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
					break;
				}

				//	Flip sense of the buffers again to refer to the buffers that the hardware isn't using (i.e. the off-screen buffers)...
				CurrentInFrame ^= 1;

				// Request buffers
				AJARequestInputBufferData RequestInputData;
				RequestInputData.bIsProgressivePicture = ::IsProgressivePicture(VideoFormat);
				if (AncBuffer || AncF2Buffer)
				{
					RequestInputData.AncBufferSize = AncBufferSize;
					RequestInputData.AncF2BufferSize = AncF2BufferSize;
				}
				if (AudioBuffer)
				{
					RequestInputData.AudioBufferSize = AudioBufferSize;
				}
				if (VideoBuffer && ::IsProgressivePicture(VideoFormat))
				{
					RequestInputData.VideoBufferSize = VideoBufferSize;
				}

				AJARequestedInputBufferData RequestedBuffer;
				{
					AJAAutoLock AutoLock(&Lock);
					if (GetOptions().CallbackInterface)
					{
						bRunning = GetOptions().CallbackInterface->OnRequestInputBuffer(RequestInputData, RequestedBuffer);
					}
				}

				// Fill the buffers
				AJAAncillaryFrameData AncillaryData;
				if (AncBuffer)
				{
#if AJA_TEST_MEMORY_BUFFER
					AJA_CHECK(AncBuffer[AncBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif

					AJA_CHECK(UseAncillary());
					AJA_CHECK(AncBufferSize > 0);
					AncillaryData.AncF2Buffer = RequestedBuffer.AncBuffer ? RequestedBuffer.AncF2Buffer : AncF2Buffer;
					NTV2_POINTER AncBufferPointer(AncillaryData.AncBuffer, AncillaryData.AncBufferSize);
					NTV2_POINTER AncF2BufferPointer(AncillaryData.AncF2Buffer, AncillaryData.AncF2BufferSize);
					bRunning = bRunning && GetDevice().DMAWriteAnc(BaseFrameIndex + CurrentInFrame, AncBufferPointer, AncF2BufferPointer);

#if AJA_TEST_MEMORY_BUFFER
					AJA_CHECK(AncBuffer[AncBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif
				}


				if (AncF2Buffer)
				{
#if AJA_TEST_MEMORY_BUFFER
					AJA_CHECK(AncBuffer[AncBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif

					AJA_CHECK(UseAncillary());
					AJA_CHECK(AncBufferSize > 0);
					AncillaryData.AncF2Buffer = RequestedBuffer.AncBuffer ? RequestedBuffer.AncF2Buffer : AncF2Buffer;
					NTV2_POINTER AncBufferPointer(AncillaryData.AncBuffer, AncillaryData.AncBufferSize);
					NTV2_POINTER AncF2BufferPointer(AncillaryData.AncF2Buffer, AncillaryData.AncF2BufferSize);
					bRunning = bRunning && GetDevice().DMAWriteAnc(BaseFrameIndex + CurrentInFrame, AncBufferPointer, AncF2BufferPointer);

#if AJA_TEST_MEMORY_BUFFER
					AJA_CHECK(AncBuffer[AncBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif
				}


				// from NTV2LLBurn::ProcessFrames
				AJAAudioFrameData AudioData;
				if (AudioBuffer)
				{
#if AJA_TEST_MEMORY_BUFFER
					AJA_CHECK(AudioBuffer[AudioBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif
					AJA_CHECK(UseAudio());
					AJA_CHECK(AudioBufferSize > 0);

					//	Read the audio position registers as close to the interrupt as possible...
					uint32_t CurrentAudioInAddress = 0;
					AJA_CHECK(GetDevice().ReadAudioLastIn(CurrentAudioInAddress, NTV2AudioSystem(Channel)));
					CurrentAudioInAddress &= ~0x3;	//	Force DWORD alignment
					CurrentAudioInAddress += AudioReadOffset;
					
					AudioData.AudioBuffer = RequestedBuffer.AudioBuffer ? RequestedBuffer.AudioBuffer : AudioBuffer;
					if (CurrentAudioInAddress < AudioInLastAddress)
					{
						//	Audio address has wrapped around the end of the buffer.
						//	Do the calculations and transfer from the last address to the end of the buffer...
						AudioBytesCaptured = AudioInWrapAddress - AudioInLastAddress;

						bRunning = bRunning && GetDevice().DMAReadAudio(AudioSystem, reinterpret_cast<ULWord*>(AudioData.AudioBuffer), AudioInLastAddress, AudioBytesCaptured);

						//	Transfer the new samples from the start of the buffer to the current address...
						bRunning = bRunning && GetDevice().DMAReadAudio(AudioSystem, reinterpret_cast<ULWord*>(AudioData.AudioBuffer + AudioBytesCaptured), AudioReadOffset, CurrentAudioInAddress - AudioReadOffset);

						AudioBytesCaptured += CurrentAudioInAddress - AudioReadOffset;
					}
					else
					{
						AudioBytesCaptured = CurrentAudioInAddress - AudioInLastAddress;

						if (AudioBytesCaptured > 0)
						{
							//	No wrap, so just perform a linear DMA from the buffer...
							bRunning = bRunning && GetDevice().DMAReadAudio(AudioSystem, reinterpret_cast<ULWord*>(AudioData.AudioBuffer), AudioInLastAddress, AudioBytesCaptured);
						}
					}

					AudioInLastAddress = CurrentAudioInAddress;
#if AJA_TEST_MEMORY_BUFFER
					AJA_CHECK(AudioBuffer[AudioBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif
				}


				AJAVideoFrameData VideoData;
				VideoData.HDROptions = GetOptions().HDROptions;
				
				if (VideoBuffer)
				{
#if AJA_TEST_MEMORY_BUFFER
					AJA_CHECK(VideoBuffer[VideoBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif

					AJA_CHECK(UseVideo());
					AJA_CHECK(VideoBufferSize > 0);
					VideoData.VideoBuffer = RequestedBuffer.VideoBuffer ? RequestedBuffer.VideoBuffer : VideoBuffer;
					bRunning = bRunning && GetDevice().DMAReadFrame(BaseFrameIndex+CurrentInFrame, reinterpret_cast<ULWord*>(VideoData.VideoBuffer), VideoBufferSize);

#if AJA_TEST_MEMORY_BUFFER
					AJA_CHECK(VideoBuffer[VideoBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif
				}

				if (!bRunning)
				{
					UE_LOGF(LogAjaCore, Error, "PingPong: Can't do the DMA frame transfer for channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
					break;
				}

				//	Determine the for the DMA timecode value
				FTimecode Timecode;
				if (UseTimecode() && !bTimecodeIssue)
				{
					FTimecode ConvertedTimecode;
					bTimecodeIssue = !Helpers::GetTimecode(GetDevicePtr(), Channel, VideoFormat, BaseFrameIndex + CurrentInFrame, GetOptions().TimecodeFormat, bLogTimecodeError, ConvertedTimecode);
					Timecode = Helpers::AdjustTimecodeForUE(Device->GetCard(), Channel, VideoFormat, ConvertedTimecode, PreviousTimecode, PreviousVerticalInterruptCount);
					PreviousTimecode = ConvertedTimecode;
				}
				else
				{
					bLogTimecodeError = false;
				}

				//	Check for dropped frames by ensuring the hardware has not started to process
				//	the buffers that were just filled....
				uint32_t ReadBackIn;
				GetDevice().GetInputFrame(Channel, ReadBackIn);
				if (ReadBackIn == BaseFrameIndex+CurrentInFrame)
				{
					++PingPongDropCount;
				}

				// Transfer to the application
				if (!bStopRequested && GetOptions().CallbackInterface)
				{
					AJAInputFrameData InputFrameData;
					InputFrameData.FramesDropped = PingPongDropCount;
					InputFrameData.Timecode = Timecode;

					if (AncBuffer)
					{
						AncillaryData.AncBuffer = AncBuffer;
						AncillaryData.AncBufferSize = AncBufferSize;
					}
					if (AncF2Buffer)
					{
						AncillaryData.AncF2Buffer = AncF2Buffer;
						AncillaryData.AncF2BufferSize = AncF2BufferSize;
					}

					if (AudioBuffer)
					{
						NTV2FrameRate FrameRate = NTV2_FRAMERATE_INVALID;
						AJA_CHECK(GetDevice().GetFrameRate(FrameRate, Channel));

						NTV2AudioRate AudioRate = NTV2_AUDIO_RATE_INVALID;
						AJA_CHECK(GetDevice().GetAudioRate(AudioRate, AudioSystem));

						ULWord NumChannels;
						AJA_CHECK(GetDevice().GetNumberAudioChannels(NumChannels, AudioSystem));

						AudioData.AudioBuffer = AudioBuffer;
						AudioData.AudioBufferSize = AudioBytesCaptured;
						AudioData.NumChannels = NumChannels;
						AudioData.AudioRate = AudioRate == NTV2_AUDIO_96K ? 96000 : 48000;
						AudioData.NumSamples = ::GetAudioSamplesPerFrame(FrameRate, AudioRate);
					}

					if (VideoBuffer)
					{
						EVerifyFormatResult Result = VerifyFormat();
						if (Result == EVerifyFormatResult::Failure)
						{
							if (!GetOptions().bAutoDetectFormat)
							{
								bRunning = false;
								break;
							}
						}

						if (Result == EVerifyFormatResult::FormatChange)
						{
							break;
						}

						if (Result == EVerifyFormatResult::Success)
						{
							VideoData.VideoFormatIndex = VideoFormat;
							VideoData.VideoBuffer = VideoBuffer;
							VideoData.VideoBufferSize = VideoBufferSize;
							VideoData.Width = FormatDescriptor.GetRasterWidth();
							VideoData.Height = FormatDescriptor.GetRasterHeight();
							VideoData.Stride = FormatDescriptor.GetBytesPerRow();
							VideoData.PixelFormat = Helpers::ConvertFrameBufferFormatToPixelFormat(FormatDescriptor.GetPixelFormat());
							VideoData.bIsProgressivePicture = ::IsProgressivePicture(VideoFormat);
							BurnTimecode(Timecode, VideoBuffer);
						}
					}

					{
						AJAAutoLock AutoLock(&Lock);
						if (GetOptions().CallbackInterface)
						{
							bRunning = GetOptions().CallbackInterface->OnInputFrameReceived(InputFrameData, AncillaryData, AudioData, VideoData);
						}
					}

					Device->OnInputFrameReceived(Channel);

					if (!bRunning)
					{
						break;
					}
				}

				//	Tell the hardware which buffers to start using at the beginning of the next frame...
				AJA_CHECK(GetDevice().SetInputFrame(Channel, BaseFrameIndex+CurrentInFrame));
			}

			if (!bStopRequested)
			{
				AJAAutoLock AutoLock(&Lock);
				if (GetOptions().CallbackInterface)
				{
					GetOptions().CallbackInterface->OnCompletion(bRunning);
				}
			}
		}

		InputChannelThread::EVerifyFormatResult InputChannelThread::VerifyFormat()
		{
			static std::string AutoCirculate = "AutoCirculate";
			static std::string PingPong = "PingPong";

			const std::string* ModeName = GetOptions().bUseAutoCirculating ? &AutoCirculate : &PingPong;

			std::string FailureReason;
			std::optional<NTV2VideoFormat> FoundFormat = Helpers::GetInputVideoFormat(GetDevicePtr(), GetOptions().TransportType, Channel, InputSource, VideoFormat, false, FailureReason);
			if (!FoundFormat.has_value())
			{ 
				bool bAutoDetectFormat = GetOptions().bAutoDetectFormat;
				const DWORD SleepMilli = 10;
				const DWORD TimeoutMilli = 2000;
				uint32_t Counter = 0;

				while (Counter * SleepMilli < TimeoutMilli)
				{
					FoundFormat = Helpers::GetInputVideoFormat(GetDevicePtr(), GetOptions().TransportType, Channel, InputSource, VideoFormat, false, FailureReason);
					if (Counter * SleepMilli > TimeoutMilli)
					{
						UE_LOGF(LogAjaCore, Error, "%s: Can't get the Channel running for channel %d on device %s.\n", ModeName->c_str(), uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
						break;
					}

					if (FoundFormat.has_value())
					{
						// If we can find the format after sleeping, the most likely cause is that only the input framerate has changed.
						if (bAutoDetectFormat)
						{
							VideoFormat = FoundFormat.value();
							Options.VideoFormatIndex = FAJAVideoFormat(FoundFormat.value());
							Device->SetFormat(Channel, FoundFormat.value());
						}
						break;
					}

					Counter += SleepMilli;
					::Sleep(SleepMilli);
				}


				if (!FoundFormat.has_value())
				{
					// Todo: Warn once, display signal lost symbol 
					if (!bAutoDetectFormat)
					{
						UE_LOGF(LogAjaCore, Error, "%s: Could not detect format for channel %d on device %s. %s\n"
							, ModeName->c_str()
							, uint32_t(Channel) + 1
							, GetDevice().GetDisplayName().c_str()
							, FailureReason.c_str()
						);
					}

					return EVerifyFormatResult::Failure;
				}
			}

			if (!Helpers::CompareFormats(DesiredVideoFormat, FoundFormat.value(), FailureReason))
			{
				if (GetOptions().bAutoDetectFormat)
				{
					DeviceConnection::CommandList CommandList(*Device);

					std::shared_ptr<InputChannelThread> SharedThis;
					if (Owner)
					{
						SharedThis = Owner->ChannelThread;
					}

					AJA::AJAVideoFormats::VideoFormatDescriptor NewFormatDescriptor = AJA::AJAVideoFormats::GetVideoFormat(FoundFormat.value());
					AJA::AJAVideoFormats::VideoFormatDescriptor OldFormatDescriptor = AJA::AJAVideoFormats::GetVideoFormat(DesiredVideoFormat);

					const bool bStandardChange = (OldFormatDescriptor.bIsInterlacedStandard != NewFormatDescriptor.bIsInterlacedStandard
						|| OldFormatDescriptor.bIsProgressiveStandard != NewFormatDescriptor.bIsProgressiveStandard
						|| OldFormatDescriptor.bIsPsfStandard != NewFormatDescriptor.bIsPsfStandard);

					if (bStandardChange && GetOptions().CallbackInterface)
					{
						Options.VideoFormatIndex = FAJAVideoFormat(FoundFormat.value());
						Device->SetFormat(Channel, FoundFormat.value());
						GetOptions().CallbackInterface->OnFormatChange(FoundFormat.value());
						return EVerifyFormatResult::FormatChange;
					}

					std::shared_ptr<IOChannelInitialize_DeviceCommand> InitializeCommand = std::make_shared<IOChannelInitialize_DeviceCommand>(Device, SharedThis);
					std::shared_ptr<IOChannelUninitialize_DeviceCommand> UninitializeCommand = std::make_shared<IOChannelUninitialize_DeviceCommand>(Device, SharedThis);

					Options.VideoFormatIndex = FAJAVideoFormat(FoundFormat.value());
					Device->SetFormat(Channel, FoundFormat.value());
					Device->AddCommand(UninitializeCommand);
					Device->AddCommand(InitializeCommand);

					return EVerifyFormatResult::FormatChange;
				}
				else
				{
					UE_LOGF(LogAjaCore, Error, "%s: The VideoFormat changed for channel %d on device %s. %s\n"
						, ModeName->c_str()
						, uint32_t(Channel) + 1
						, GetDevice().GetDisplayName().c_str()
						, FailureReason.c_str()
					);

					// No format change allowed when not in autodetect mode.
					return EVerifyFormatResult::Failure;
				}
			}

			return EVerifyFormatResult::Success;
		}
	}
}
