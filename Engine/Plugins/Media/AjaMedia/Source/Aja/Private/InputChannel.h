// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChannelBase.h"
#include "CoreMinimal.h"

namespace AJA
{
	namespace Private
	{
		class InputChannelThread;

		/* InputChannel definition
		*****************************************************************************/
		class InputChannel
		{
		public:
			InputChannel() = default;
			InputChannel(const InputChannel&) = delete;
			InputChannel& operator=(const InputChannel&) = delete;

			bool Initialize(const AJADeviceOptions& InDevice, const AJAInputOutputChannelOptions& Options);
			void Uninitialize();

			AUTOCIRCULATE_STATUS GetCurrentAutoCirculateStatus() const;
			const AJAInputOutputChannelOptions& GetOptions() const;
			const AJADeviceOptions& GetDeviceOptions() const;


		private:
			std::shared_ptr<InputChannelThread> ChannelThread;
			std::shared_ptr<DeviceConnection> Device;
			std::weak_ptr<IOChannelInitialize_DeviceCommand> InitializeCommand;
			friend class InputChannelThread;
		};

		/* InputChannelThread definition
		*****************************************************************************/
		class InputChannelThread : public ChannelThreadBase
		{
			using Super = ChannelThreadBase;
		public:
			InputChannelThread(InputChannel* InOwner, const AJADeviceOptions& InDevice, const AJAInputOutputChannelOptions& InOptions);
			virtual ~InputChannelThread();

			virtual void DeviceThread_Destroy(DeviceConnection::CommandList& InCommandList) override;

		protected:
			virtual bool DeviceThread_ConfigureAnc(DeviceConnection::CommandList& InCommandList) override;
			virtual bool DeviceThread_ConfigureAudio(DeviceConnection::CommandList& InCommandList) override;
			virtual bool DeviceThread_ConfigureVideo(DeviceConnection::CommandList& InCommandList) override;

			virtual bool DeviceThread_ConfigureAutoCirculate(DeviceConnection::CommandList& InCommandList) override;
			virtual bool DeviceThread_ConfigurePingPong(DeviceConnection::CommandList& InCommandList) override;

			virtual void Thread_AutoCirculateLoop() override;
			virtual void Thread_PingPongLoop() override;

			uint8_t* AncBuffer;
			uint8_t* AncF2Buffer;
			uint8_t* AudioBuffer;
			uint8_t* VideoBuffer;

			uint32_t PingPongDropCount;

		private:
			enum class EVerifyFormatResult : uint8_t
			{
				Failure,
				Success,
				FormatChange
			};

			EVerifyFormatResult VerifyFormat();

		private:
			InputChannel* Owner = nullptr;

		};
	}
}