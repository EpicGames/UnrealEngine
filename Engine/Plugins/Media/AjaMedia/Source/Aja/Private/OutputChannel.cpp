// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputChannel.h"

#include "GPUTextureTransfer.h"
#include "RenderingThread.h"
#include "SSE2MemCpy.h"
#include <stdlib.h>

DECLARE_FLOAT_COUNTER_STAT(TEXT("AJA Audio Delay (s)"), STAT_AjaMediaCapture_Audio_Delay, STATGROUP_Media);

#include "Misc/ScopeLock.h"

static TAutoConsoleVariable<int32> CVarAjaPreRollMethod(
	TEXT("Aja.PrerollMethod"), 0,
	TEXT("Preroll method 0: Legacy method where we preroll 3 empty frames. Preroll method 1: We preroll engine frames as they come in."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAjaPingPongVersion(
	TEXT("Aja.PingPongVersion"), 0,
	TEXT("Version 0: Legacy, Version 1: Experimental with reduced waiting."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarAjaShowDebugMarkers(
	TEXT("Aja.EnableDebugMarkers"), false,
	TEXT("Enable to let Aja Output Channel trace more information when doing a media capture, such as bookmarks for Vertical Interrupts etc."),
	ECVF_RenderThreadSafe);

namespace AJA
{
	namespace Private
	{
		/* OutputChannel implementation
		*****************************************************************************/
		bool OutputChannel::Initialize(const AJADeviceOptions& InDeviceOption, const AJAInputOutputChannelOptions& InOptions)
		{
			TPromise<void> CompletionPromise;
			Uninitialize(MoveTemp(CompletionPromise));

			ChannelThread.reset(new OutputChannelThread(InDeviceOption, InOptions));

			bool bResult = ChannelThread->CanInitialize();
			if (bResult)
			{
				Device = DeviceCache::GetDevice(InDeviceOption);
				std::shared_ptr<IOChannelInitialize_DeviceCommand> SharedCommand(new IOChannelInitialize_DeviceCommand(Device, ChannelThread));
				InitializeCommand = SharedCommand;
				Device->AddCommand(std::static_pointer_cast<DeviceCommand>(SharedCommand));
			}

			return bResult;
		}

		void OutputChannel::Uninitialize(TPromise<void> CompletionPromise)
		{
			if (ChannelThread)
			{
				if (Device)
				{
					// We have to make sure the format is cleared upon uninitialization in case an initialization call happens soon after.
					Device->ClearFormat();
				}

				std::shared_ptr<IOChannelInitialize_DeviceCommand> SharedCommand = InitializeCommand.lock();
				if (SharedCommand && Device)
				{
					Device->RemoveCommand(std::static_pointer_cast<DeviceCommand>(SharedCommand));
				}

				ChannelThread->Uninitialize(); // delete is done in IOChannelUninitialize_DeviceCommand completed

				if (Device)
				{
					std::shared_ptr<IOChannelUninitialize_DeviceCommand> UninitializeCommand;
					UninitializeCommand.reset(new IOChannelUninitialize_DeviceCommand(Device, ChannelThread, MoveTemp(CompletionPromise)));
					Device->AddCommand(std::static_pointer_cast<DeviceCommand>(UninitializeCommand));
				}
				else
				{
					// No need to wait in this case.
					CompletionPromise.SetValue();
				}
			}
			else
			{
				CompletionPromise.SetValue();
			}

			InitializeCommand.reset();
			ChannelThread.reset();
			Device.reset();
		}

		bool OutputChannel::SetAncillaryFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* InAncillaryBuffer, uint32_t InAncillaryBufferSize)
		{
			if (ChannelThread)
			{
				return ChannelThread->SetAncillaryFrameData(InFrameData, InAncillaryBuffer, InAncillaryBufferSize);
			}
			return false;
		}

		bool OutputChannel::SetAudioFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* InAudioBuffer, uint32_t InAudioBufferSize)
		{
			if (ChannelThread)
			{
				return ChannelThread->SetAudioFrameData(InFrameData, InAudioBuffer, InAudioBufferSize);
			}
			return false;
		}

		bool OutputChannel::SetVideoFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* InVideoBuffer, uint32_t InVideoBufferSize)
		{
			if (ChannelThread)
			{
				return ChannelThread->SetVideoFrameData(InFrameData, InVideoBuffer, InVideoBufferSize);
			}
			return false;
		}

		bool OutputChannel::DMAWriteAudio(const uint8_t* InAudioBufer, int32_t InAudioBufferSize)
		{
			if (ChannelThread)
			{
				return ChannelThread->DMAWriteAudio(InAudioBufer, InAudioBufferSize);
			}
			return false;
		}

		bool OutputChannel::SetVideoFrameData(const AJAOutputFrameBufferData& InFrameData, FRHITexture* RHITexture)
		{
			if (ChannelThread)
			{
				return ChannelThread->SetVideoFrameData(InFrameData, RHITexture);
			}
			return false;
		}

		bool OutputChannel::ShouldCaptureThisFrame(::FTimecode Timeocode, uint32 SourceFrameNumber)
		{
			if (ChannelThread)
			{
				return ChannelThread->ShouldCaptureThisFrame(Timeocode, SourceFrameNumber);
			}
			return false;
		}

		bool OutputChannel::GetOutputDimension(uint32_t& OutWidth, uint32_t& OutHeight) const
		{
			NTV2FormatDescriptor FormatDescriptor = ChannelThread->FormatDescriptor;
			bool bResult = FormatDescriptor.IsValid();
			if (bResult)
			{
				OutWidth = ChannelThread->FormatDescriptor.GetRasterWidth();
				OutHeight = ChannelThread->FormatDescriptor.GetRasterHeight();
			}
			return bResult;
		}

		int32_t OutputChannel::GetNumAudioSamplesPerFrame(const AJAOutputFrameBufferData& InFrameData) const
		{
			if (ChannelThread && Device && Device->GetCard())
			{
				NTV2FrameRate FrameRate = NTV2_FRAMERATE_INVALID;
				AJA_CHECK(Device->GetCard()->GetFrameRate(FrameRate, ChannelThread->Channel));

				// @todo: Check if 1080p60, 1080p5994 or 1080p50 and pass SMTPE 372M flag
				return ::GetAudioSamplesPerFrame(FrameRate, NTV2_AUDIO_48K, InFrameData.FrameIdentifier);
			}

			return 0;
		}

		/* Frame implementation
		*****************************************************************************/
		OutputChannelThread::Frame::Frame()
			: AncBuffer(nullptr)
			, AncF2Buffer(nullptr)
			, AudioBuffer(nullptr)
			, VideoBuffer(nullptr)
		{
			Clear();
		}

		OutputChannelThread::Frame::~Frame()
		{
			if (AncBuffer)
			{
				AJAMemory::FreeAligned(AncBuffer);
			}
			if (AncF2Buffer)
			{
				AJAMemory::FreeAligned(AncF2Buffer);
			}
			if (AudioBuffer)
			{
				AJAMemory::FreeAligned(AudioBuffer);
			}
			if (VideoBuffer)
			{
				AJAMemory::FreeAligned(VideoBuffer);
			}
		}

		void OutputChannelThread::Frame::Clear()
		{
			CopiedAncBufferSize = 0;
			CopiedAncF2BufferSize = 0;
			CopiedAudioBufferSize = 0;
			CopiedVideoBufferSize = 0;
			FrameIdentifier = AJAOutputFrameBufferData::InvalidFrameIdentifier;
			FrameIdentifierF2 = AJAOutputFrameBufferData::InvalidFrameIdentifier;
			bAncLineFilled = false;
			bAncF2LineFilled = false;
			bAudioLineFilled = false;
			bVideoLineFilled = false;
			bVideoF2LineFilled = false;
		}

		/* OutputChannelThread implementation
		*****************************************************************************/
		OutputChannelThread::OutputChannelThread(const AJADeviceOptions& InDevice, const AJAInputOutputChannelOptions& InOptions)
			: Super(InDevice, InOptions)
			, PingPongDropCount(0)
			, LostFrameCounter(0)
			, bIsSSE2Available(IsSSE2Available())
			, bIsFieldAEven(true)
			, bIsFirstFetch(true)
			, bInterlacedTest_ExpectFirstLineToBeWhite(false)
			, InterlacedTest_FrameCounter(0)
		{
			if (FGPUTextureTransferModule::Get().IsEnabled() && FGPUTextureTransferModule::Get().IsInitialized())
			{
				TextureTransfer = FGPUTextureTransferModule::Get().GetTextureTransfer();
			}

			if (InOptions.OutputNumberOfBuffers > 0)
			{
				AllFrames.reserve(InOptions.OutputNumberOfBuffers);
				for (uint32_t Index = 0; Index < InOptions.OutputNumberOfBuffers; ++Index)
				{
					Frame* NewFrame = new Frame();
					AllFrames.push_back(NewFrame);
					FrameReadyToWrite.push_back(NewFrame);
				}
			}

			FrameAvailableEvent = FPlatformProcess::GetSynchEventFromPool();
			bOutputWithReduceWaiting = CVarAjaPingPongVersion.GetValueOnAnyThread() == 1;
			bTraceDebugMarkers = CVarAjaShowDebugMarkers.GetValueOnAnyThread();
		}

		OutputChannelThread::~OutputChannelThread()
		{
			FPlatformProcess::ReturnSynchEventToPool(FrameAvailableEvent);
		}

		bool OutputChannelThread::CanInitialize() const
		{
			return Super::CanInitialize() && AllFrames.size() >= 1;
		}

		void OutputChannelThread::Uninitialize()
		{
			ChannelThreadBase::Uninitialize();

			if (InterruptEventHandle.IsValid())
			{
				Device->ClearOnInterruptEvent(Channel, InterruptEventHandle);
			}

			if (TextureTransfer && GetOptions().bUseGPUDMA)
			{
				std::vector<uint8_t*> BuffersToClear;
				BuffersToClear.reserve(AllFrames.size());
				std::transform(AllFrames.cbegin(), AllFrames.cend(), std::back_inserter(BuffersToClear), [](Frame* InFrame) { return InFrame->VideoBuffer; });

				// Make sure to unregister those buffers on the render thread to avoid a crash in the GPUDirect SDK.
				ENQUEUE_RENDER_COMMAND(ClearGPUDirectBuffers)(
				[BuffersToClear = std::move(BuffersToClear), TextureTransferPtr = TextureTransfer](FRHICommandListImmediate& RHICmdList)
				{
					if (TextureTransferPtr)
					{
						for (uint8* BufferAddress : BuffersToClear)
						{
							TextureTransferPtr->UnregisterBuffer(BufferAddress);
						}
					}
				});
			}
		}

		void OutputChannelThread::DeviceThread_Destroy(DeviceConnection::CommandList& InCommandList)
		{
			for (Frame* IttFrame : AllFrames)
			{
				delete IttFrame;
			}
			AllFrames.clear();
			Super::DeviceThread_Destroy(InCommandList);
		}

		// For Progressive and PSF. Find the frame that have the same identifier. (the image may have been given but the anc and audio is incoming)
		// For Interlaced, there is 2 options. Use Timecode to identify which is the odd and even field.
		OutputChannelThread::Frame* OutputChannelThread::FetchAvailableWritingFrame(const AJAOutputFrameBufferData& InFrameData)
		{
			const bool bIsProgressive = ::IsProgressivePicture(VideoFormat) || Options.bOutputInterlaceAsProgressive;

			Frame* AvailableWritingFrame = nullptr;
			{
				AJAAutoLock AutoLock(&FrameLock);

				bool bInitFrameIdentifier = false;
				{
					// Find a matching frame identifier
					for (auto Begin = FrameReadyToWrite.begin(); Begin != FrameReadyToWrite.end(); ++Begin)
					{
						Frame* FrameItt = *Begin;
						if (bIsProgressive && FrameItt->FrameIdentifier == InFrameData.FrameIdentifier)
						{
							AvailableWritingFrame = FrameItt;
							break;
						}
						if (!bIsProgressive && FrameItt->FrameIdentifierF2 == InFrameData.FrameIdentifier)
						{
							AvailableWritingFrame = FrameItt;
							break;
						}
					}
				}

				if (AvailableWritingFrame == nullptr && !bIsProgressive)
				{
					// Find a frame where the even frame is filled but not the odd frame and have the same timecode (only in interlaced)
					for (auto Begin = FrameReadyToWrite.begin(); Begin != FrameReadyToWrite.end(); ++Begin)
					{
						Frame* FrameItt = *Begin;
						if (FrameItt->FrameIdentifier != AJAOutputFrameBufferData::InvalidFrameIdentifier && FrameItt->FrameIdentifierF2 == AJAOutputFrameBufferData::InvalidFrameIdentifier)
						{
							if (FrameItt->FrameIdentifier + 1 == InFrameData.FrameIdentifier)
							{
								if (GetOptions().bOutputInterlacedFieldsTimecodeNeedToMatch)
								{
									if (FrameItt->Timecode == InFrameData.Timecode)
									{
										AvailableWritingFrame = FrameItt;
										bInitFrameIdentifier = true;
										break;
									}
								}
								else
								{
									AvailableWritingFrame = FrameItt;
									bInitFrameIdentifier = true;
									break;
								}
							}
						}
					}
				}

				// Find a new empty frames

				if (AvailableWritingFrame == nullptr && !bIsProgressive && !GetOptions().bOutputInterlacedFieldsTimecodeNeedToMatch)
				{
					if (!GetOptions().bOutputInterlaceOnEvenFrames)
					{
						if (!InFrameData.bEvenFrame)
						{
							// If the incoming frame is interlaced and we didn't find the correspondent even field
							// do not create a new field. Drop it.
							++LostFrameCounter;
							return nullptr;
						}
					}
				}

				if (AvailableWritingFrame == nullptr)
				{
					// Find an empty frame
					for (auto Begin = FrameReadyToWrite.begin(); Begin != FrameReadyToWrite.end(); ++Begin)
					{
						Frame* FrameItt = *Begin;
						if (FrameItt->FrameIdentifier == AJAOutputFrameBufferData::InvalidFrameIdentifier)
						{
							AJA_CHECK(FrameItt->FrameIdentifierF2 == AJAOutputFrameBufferData::InvalidFrameIdentifier);
							AvailableWritingFrame = FrameItt;
							bInitFrameIdentifier = true;

							break;
						}
					}
				}

				if (AvailableWritingFrame == nullptr)
				{
					// Take the oldest frame not yet written
					if (FrameReadyToWrite.size() > 0)
					{
						AvailableWritingFrame = FrameReadyToWrite[0];
						uint32_t OldestNumber = AvailableWritingFrame->FrameIdentifier;

						auto Begin = FrameReadyToWrite.begin();
						++Begin;
						for (; Begin != FrameReadyToWrite.end(); ++Begin)
						{
							Frame* FrameItt = *Begin;
							if (FrameItt->FrameIdentifier > OldestNumber)
							{
								AvailableWritingFrame = FrameItt;
								OldestNumber = FrameItt->FrameIdentifier;
							}
						}

						// This is not thread-safe with but FetchAvailableWritingFrame & IsFrameReadyToBeRead are both in inside a lock.
						//So even if we are currently writing in a buffer, it will be cleared and will not be available to be added to the read list.
						AvailableWritingFrame->Clear();
						bInitFrameIdentifier = true;

						++LostFrameCounter;
					}
				}

				//if (AvailableWritingFrame == nullptr)
				//{
				//	// Take the newest frame that is ready
				//	if (FrameReadyToRead.size() > 0)
				//	{
				//		AvailableWritingFrame = FrameReadyToRead[0];
				//		uint32_t NewestNumber = AvailableWritingFrame->FrameIdentifier;
				//		auto NewestItt = std::begin(FrameReadyToRead);

				//		auto Begin = NewestItt;
				//		++Begin;
				//		auto End = std::end(FrameReadyToRead);
				//		for (; Begin != End; ++Begin)
				//		{
				//			Frame* FrameItt = *Begin;
				//			if (FrameItt->FrameIdentifier < NewestNumber)
				//			{
				//				AvailableWritingFrame = FrameItt;
				//				NewestNumber = FrameItt->FrameIdentifier;
				//				NewestItt = Begin;
				//			}
				//		}

				//		// This is not thread-safe (see above)
				//		AvailableWritingFrame->Clear();
				//		bInitFrameIdentifier = true;

				//		FrameReadyToWrite.push_back(AvailableWritingFrame);
				//		FrameReadyToRead.erase(NewestItt);
				//		++LostFrameCounter;
				//	}
				//}

				if (AvailableWritingFrame && bInitFrameIdentifier)
				{
					if (AvailableWritingFrame->FrameIdentifier == AJAOutputFrameBufferData::InvalidFrameIdentifier)
					{
						AvailableWritingFrame->FrameIdentifier = InFrameData.FrameIdentifier;
						AvailableWritingFrame->Timecode = InFrameData.Timecode;
						if (bIsProgressive)
						{
							AvailableWritingFrame->FrameIdentifierF2 = InFrameData.FrameIdentifier;
						}
					}
					else
					{
						AvailableWritingFrame->FrameIdentifierF2 = InFrameData.FrameIdentifier;
						AvailableWritingFrame->Timecode2 = InFrameData.Timecode;
					}
				}
			}

			return AvailableWritingFrame;
		}

		OutputChannelThread::Frame* OutputChannelThread::Thread_FetchAvailableReadingFrame()
		{
			Frame* AvailableReadingFrame = nullptr;

			AJAAutoLock AutoLock(&FrameLock);
			if (!FrameReadyToRead.empty())
			{
				// Take the oldest frame
				AvailableReadingFrame = FrameReadyToRead[0];
				uint32_t OldestNumber = AvailableReadingFrame->FrameIdentifier;
				auto OldestItt = std::begin(FrameReadyToRead);

				auto Begin = OldestItt;
				++Begin;
				auto End = std::end(FrameReadyToRead);
				for (; Begin != End; ++Begin)
				{
					Frame* FrameItt = *Begin;
					if (FrameItt->FrameIdentifier < OldestNumber)
					{
						AvailableReadingFrame = FrameItt;
						OldestNumber = FrameItt->FrameIdentifier;
						OldestItt = Begin;
					}
				}

				FrameReadyToRead.erase(OldestItt);
			}

			return AvailableReadingFrame;
		}

		bool OutputChannelThread::IsFrameReadyToBeRead(Frame* InFrame)
		{
			if (InFrame->FrameIdentifier == AJAOutputFrameBufferData::InvalidFrameIdentifier)
			{
				InFrame->Clear();
				return false;
			}
			if (UseAncillary() && (!InFrame->bAncLineFilled || InFrame->CopiedAncBufferSize == 0))
			{
				return false;
			}
			if (UseAncillaryField2() && (!InFrame->bAncF2LineFilled || InFrame->CopiedAncF2BufferSize == 0))
			{
				return false;
			}
			// When writing audio directly, no need to check if the audio line is filled since we don't use it.
			if (UseAudio() && !InFrame->bAudioLineFilled && !GetOptions().bDirectlyWriteAudio)
			{
				return false;
			}
			if (UseVideo() && (!InFrame->bVideoF2LineFilled || !InFrame->bVideoLineFilled || InFrame->CopiedVideoBufferSize == 0))
			{
				return false;
			}
			return true;
		}

		void OutputChannelThread::PushWhenFrameReady(Frame* InFrame)
		{
			AJAAutoLock AutoLock(&FrameLock);
			if (IsFrameReadyToBeRead(InFrame))
			{
				auto FoundItt = std::find(std::begin(FrameReadyToWrite), std::end(FrameReadyToWrite), InFrame);
				if (FoundItt != std::end(FrameReadyToWrite))
				{
					FrameReadyToWrite.erase(FoundItt);
					FrameReadyToRead.push_back(InFrame);
					FrameAvailableEvent->Trigger();
				}
			}
		}

		bool OutputChannelThread::SetAncillaryFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* InAncillaryBuffer, uint32_t InAncillaryBufferSize)
		{
			const bool bIsProgressive = ::IsProgressivePicture(VideoFormat) || Options.bOutputInterlaceAsProgressive;

			if (!UseAncillary())
			{
				return false;
			}

			if (InAncillaryBuffer == nullptr || InAncillaryBufferSize > AncBufferSize || (!bIsProgressive && InAncillaryBufferSize > AncF2BufferSize))
			{
				UE_LOGF(LogAjaCore, Error, "SetFrameData: Can't set the ancillary. The buffer is invalid or the buffer size is not the same as the AJA for output channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
				return false;
			}

			Frame* AvailableWritingFrame = FetchAvailableWritingFrame(InFrameData);
			if (!AvailableWritingFrame)
			{
				return false;
			}

#if AJA_TEST_MEMORY_BUFFER
			AJA_CHECK(AvailableWritingFrame->AncBuffer[AncBufferSize] == AJA_TEST_MEMORY_END_TAG);
			AJA_CHECK(AncF2BufferSize == 0 || AvailableWritingFrame->AncF2Buffer[AncF2BufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif

			if (bIsProgressive)
			{
				memcpy(AvailableWritingFrame->AncBuffer, InAncillaryBuffer, InAncillaryBufferSize);
				AvailableWritingFrame->bAncLineFilled = true;
				AvailableWritingFrame->bAncF2LineFilled = true;
				AvailableWritingFrame->CopiedAncBufferSize = InAncillaryBufferSize;
			}
			else
			{
				bool bField1 = AvailableWritingFrame->FrameIdentifier == InFrameData.FrameIdentifier;
				if (bField1)
				{
					memcpy(AvailableWritingFrame->AncBuffer, InAncillaryBuffer, InAncillaryBufferSize);
					AvailableWritingFrame->bAncLineFilled = true;
					AvailableWritingFrame->CopiedAncBufferSize = InAncillaryBufferSize;
				}
				else
				{
					memcpy(AvailableWritingFrame->AncF2Buffer, InAncillaryBuffer, InAncillaryBufferSize);
					AvailableWritingFrame->bAncF2LineFilled = true;
					AvailableWritingFrame->CopiedAncF2BufferSize = InAncillaryBufferSize;
				}
			}

#if AJA_TEST_MEMORY_BUFFER
			AJA_CHECK(AvailableWritingFrame->AncBuffer[AncBufferSize] == AJA_TEST_MEMORY_END_TAG);
			AJA_CHECK(AncF2BufferSize == 0 || AvailableWritingFrame->AncF2Buffer[AncF2BufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif

			PushWhenFrameReady(AvailableWritingFrame);

			return true;
		}

		bool OutputChannelThread::SetAudioFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* InAudioBuffer, uint32_t InAudioBufferSize)
		{
			if (!UseAudio())
			{
				return false;
			}
			
			if (InAudioBufferSize > AudioBufferSize)
			{
				UE_LOGF(LogAjaCore, Error, "SetAudioFrameData: Can't set the audio. The buffer size is not the same as the AJA Audio Format for output channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
				return false;
			}

			Frame* AvailableWritingFrame = FetchAvailableWritingFrame(InFrameData);
			if (!AvailableWritingFrame)
			{
				return false;
			}

#if AJA_TEST_MEMORY_BUFFER
			AJA_CHECK(AvailableWritingFrame->Audio[AudioBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif

			if (InAudioBuffer && InAudioBufferSize != 0 && ensure(AvailableWritingFrame->AudioBuffer))
			{
				memcpy(AvailableWritingFrame->AudioBuffer, InAudioBuffer, InAudioBufferSize);
			}

			AvailableWritingFrame->bAudioLineFilled = true;
			AvailableWritingFrame->CopiedAudioBufferSize = InAudioBufferSize;
			PushWhenFrameReady(AvailableWritingFrame);

			return true;
		}

		bool OutputChannelThread::SetVideoFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* InVideoBuffer, uint32_t InVideoBufferSize)
		{
			const bool bIsProgressive = ::IsProgressivePicture(VideoFormat) || Options.bOutputInterlaceAsProgressive;

			if (!UseVideo())
			{
				return false;
			}

			if (InVideoBuffer == nullptr || InVideoBufferSize > VideoBufferSize)
			{
				UE_LOGF(LogAjaCore, Error, "SetVideoFrameData: Can't set the video. The buffer is invalid or the buffer size is not the same as the AJA Video Format for output channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
				return false;
			}

			Frame* AvailableWritingFrame = FetchAvailableWritingFrame(InFrameData);
			if (!AvailableWritingFrame)
			{
				if (bTraceDebugMarkers)
				{
					TRACE_BOOKMARK(TEXT("Aja: Dropped frame %d"), InFrameData.FrameIdentifier);
				}
				return false;
			}

			ULWord Height = FormatDescriptor.GetRasterHeight();
			ULWord Stride = FormatDescriptor.GetBytesPerRow();
			AJA_CHECK(InVideoBufferSize <= Stride*Height);

#if AJA_TEST_MEMORY_BUFFER
			AJA_CHECK(AvailableWritingFrame->VideoBuffer[VideoBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif

			if (bIsProgressive)
			{
				// Make sure SSE2 is supported and everything is correctly aligned otherwise fall back on regular memcpy
				if (bIsSSE2Available && IsCorrectlyAlignedForSSE2MemCpy(AvailableWritingFrame->VideoBuffer, InVideoBuffer, InVideoBufferSize))
				{
					SSE2MemCpy(AvailableWritingFrame->VideoBuffer, InVideoBuffer, InVideoBufferSize);
				}
				else
				{
					memcpy(AvailableWritingFrame->VideoBuffer, InVideoBuffer, InVideoBufferSize);
				}
				AvailableWritingFrame->bVideoLineFilled = true;
				AvailableWritingFrame->bVideoF2LineFilled = true;
			}
			else
			{
				// only write the even or odd frame
				bool bField1 = AvailableWritingFrame->FrameIdentifier == InFrameData.FrameIdentifier;
				for (ULWord IndexY = bField1 ? 0 : 1; IndexY < Height; IndexY += 2)
				{
					memcpy(AvailableWritingFrame->VideoBuffer + (Stride*IndexY), InVideoBuffer + (Stride*IndexY), Stride);
				}
				if (bField1)
				{
					AvailableWritingFrame->bVideoLineFilled = true;
				}
				else
				{
					AvailableWritingFrame->bVideoF2LineFilled = true;
				}
			}

#if AJA_TEST_MEMORY_BUFFER
			AJA_CHECK(AvailableWritingFrame->VideoBuffer[VideoBufferSize] == AJA_TEST_MEMORY_END_TAG);
#endif

			AvailableWritingFrame->CopiedVideoBufferSize = InVideoBufferSize;

			PushWhenFrameReady(AvailableWritingFrame);

			if (bTraceDebugMarkers)
			{
				TRACE_BOOKMARK(TEXT("Aja: Pushed [%d] (Frame %d)"), InFrameData.FrameIdentifier, InFrameData.Timecode.Frames);
			}

			return true;
		}

		bool OutputChannelThread::SetVideoFrameData(const AJAOutputFrameBufferData& InFrameData, FRHITexture* RHITexture)
		{
			const bool bIsProgressive = ::IsProgressivePicture(VideoFormat) || Options.bOutputInterlaceAsProgressive;

			if (!bIsProgressive)
			{
				UE_LOGF(LogAjaCore, Error, "SetVideoFrameData: Can't set the video, GPUTextureTransfer is not supported with interlaced video for output channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
				return false;
			}

			if (!UseVideo())
			{
				return false;
			}

			if (!RHITexture)
			{
				UE_LOGF(LogAjaCore, Error, "SetVideoFrameData: Can't set the video. The RHI texture is invalid for output channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
				return false;
			}

			if (!TextureTransfer || !GetOptions().bUseGPUDMA)
			{
				UE_LOGF(LogAjaCore, Error, "SetVideoFrameData: Can't set the video. GPU DMA was not setup correctly for output channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
				return false;
			}


			Frame* AvailableWritingFrame = FetchAvailableWritingFrame(InFrameData);
			if (!AvailableWritingFrame)
			{
				return false;
			}
			 
			// Todo jroy: Verify if Width needs to be fixed, by using the GPUTextureTransfer register struct directly, same for pixel format.
			const ULWord Stride = FormatDescriptor.GetBytesPerRow();
			ULWord Width = Stride / 4;

			if (GetOptions().PixelFormat == EPixelFormat::PF_10BIT_YCBCR)
			{
				// YUV210 has a different stride.
				Width /= 4;
			}

			const ULWord Height = FormatDescriptor.GetRasterHeight();

			if (!bDMABuffersRegistered)
			{
				for (Frame* IttFrame : AllFrames)
				{
					UE::GPUTextureTransfer::FRegisterDMABufferArgs Args;
					Args.Height = Height;
					Args.Width = Width;
					Args.Stride = Stride;
					Args.Buffer = IttFrame->VideoBuffer;

					if (GetOptions().PixelFormat == EPixelFormat::PF_8BIT_YCBCR || GetOptions().PixelFormat == EPixelFormat::PF_8BIT_ARGB
						|| GetOptions().PixelFormat == EPixelFormat::PF_10BIT_RGB) // 10 Bit RGB should be considered as 8 bit as far as GPUDirect is concerned.
					{

						Args.PixelFormat = UE::GPUTextureTransfer::EPixelFormat::PF_8Bit;
					}
					else
					{
						Args.PixelFormat = UE::GPUTextureTransfer::EPixelFormat::PF_10Bit;
					}

					TextureTransfer->RegisterBuffer(Args);
				}

				bDMABuffersRegistered = true;
			}

			AvailableWritingFrame->bVideoLineFilled = true;
			AvailableWritingFrame->bVideoF2LineFilled = true;

			const UE::GPUTextureTransfer::ETransferDirection Direction = UE::GPUTextureTransfer::ETransferDirection::GPU_TO_CPU;
			TextureTransfer->TransferTexture(AvailableWritingFrame->VideoBuffer, (FRHITexture*)RHITexture, Direction);

			AvailableWritingFrame->CopiedVideoBufferSize = VideoBufferSize;
			PushWhenFrameReady(AvailableWritingFrame);

			return true;
		}

		bool OutputChannelThread::ShouldCaptureThisFrame(::FTimecode Timecode, uint32 SourceFrameNumber)
		{
			const bool bInterlace = !::IsProgressivePicture(VideoFormat) && !Options.bOutputInterlaceAsProgressive;

			if (bInterlace && Timecode.Frames % 2 != 0 && GetOptions().bOutputInterlaceOnEvenFrames)
			{
				// Odd frame should always have a matching frame.
				// Find a frame where the even frame is filled but not the odd frame.
				for (Frame* Frame : FrameReadyToWrite)
				{
					if (Frame->FrameIdentifier != AJAOutputFrameBufferData::InvalidFrameIdentifier && Frame->FrameIdentifierF2 == AJAOutputFrameBufferData::InvalidFrameIdentifier)
					{
						if (Frame->FrameIdentifier + 1 == SourceFrameNumber)
						{
							return true;
						}
					}
				}

				if (!FrameReadyToWrite.size())
				{
					UE_LOGF(LogAjaCore, Verbose, "Skipping output because no writable frame is available.");
				}
				else
				{
					for (Frame* Frame : FrameReadyToWrite)
					{
						UE_LOGF(LogAjaCore, VeryVerbose, "\t F1: %d, F2: %d", Frame->FrameIdentifier, Frame->FrameIdentifierF2);
					}
				}

				// Dont output an interlace frame starting with an odd frame.
				return false;
			}

			return true;
		}

		bool OutputChannelThread::DeviceThread_ConfigureAnc(DeviceConnection::CommandList& InCommandList)
		{
			bool bResult = Super::DeviceThread_ConfigureAnc(InCommandList);

			if (AncBufferSize > 0 || AncF2BufferSize > 0)
			{
				AJAAutoLock AutoLock(&FrameLock);
				for (Frame* IttFrame : AllFrames)
				{
					AJA_CHECK(IttFrame->AncBuffer == nullptr);
					AJA_CHECK(IttFrame->AncF2Buffer == nullptr);

					if (AncBufferSize > 0)
					{
						AJA_CHECK(UseAncillary());
#if AJA_TEST_MEMORY_BUFFER
						IttFrame->AncBuffer = (uint8_t*)AJAMemory::AllocateAligned(AncBufferSize + 1, AJA_PAGE_SIZE);
						IttFrame->AncBuffer[AncBufferSize] = AJA_TEST_MEMORY_END_TAG;
#else
						IttFrame->AncBuffer = (uint8_t*)AJAMemory::AllocateAligned(AncBufferSize, AJA_PAGE_SIZE);
#endif
					}

					if (AncF2BufferSize > 0)
					{
						AJA_CHECK(UseAncillaryField2());
#if AJA_TEST_MEMORY_BUFFER
						IttFrame->AncF2Buffer = (uint8_t*)AJAMemory::AllocateAligned(AncF2BufferSize + 1, AJA_PAGE_SIZE);
						IttFrame->AncF2Buffer[AncF2BufferSize] = AJA_TEST_MEMORY_END_TAG;
#else
						IttFrame->AncF2Buffer = (uint8_t*)AJAMemory::AllocateAligned(AncF2BufferSize, AJA_PAGE_SIZE);
#endif
					}
				}
			}
			return bResult;
		}

		bool OutputChannelThread::DeviceThread_ConfigureAudio(DeviceConnection::CommandList& InCommandList)
		{
			bool bResult = Super::DeviceThread_ConfigureAudio(InCommandList);
			if(bResult && UseAudio())
			{
				NTV2FrameRate CardFrameRate = NTV2_FRAMERATE_INVALID;
				AJA_CHECK(Device->GetCard()->GetFrameRate(CardFrameRate, Channel));
				NumSamplesPerFrame = ::GetAudioSamplesPerFrame(CardFrameRate, NTV2_AUDIO_48K);

				if (ensureMsgf(AudioBufferSize > 0, TEXT("Audio could not be initialized.")))
				{
					if (!GetOptions().bDirectlyWriteAudio)
					{
						AJAAutoLock AutoLock(&FrameLock);
						for (Frame* IttFrame : AllFrames)
						{
							AJA_CHECK(IttFrame->AudioBuffer == nullptr);
		#if AJA_TEST_MEMORY_BUFFER
							IttFrame->AudioBuffer = (uint8_t*)AJAMemory::AllocateAligned(AudioBufferSize + 1, AJA_PAGE_SIZE);
							IttFrame->AudioBuffer[AudioBufferSize] = AJA_TEST_MEMORY_END_TAG;
		#else
							IttFrame->AudioBuffer = (uint8_t*)AJAMemory::AllocateAligned(AudioBufferSize, AJA_PAGE_SIZE);
		#endif
							memset(IttFrame->AudioBuffer, 0, AudioBufferSize);
						}
					}

					if (GetOptions().bUseAutoCirculating)
					{
						bResult &= GetDevice().StartAudioOutput(AudioSystem);
						if (bResult == false)
						{
							UE_LOGF(LogAjaCore, Error, "ConfigureAudio: Could not start audio output for audio system %d for channel %d on device %s.", uint32_t(AudioSystem), uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
						}
					}
				}
			}

			return bResult;
		}

		bool OutputChannelThread::DeviceThread_ConfigureVideo(DeviceConnection::CommandList& InCommandList)
		{
			bool bResult = Super::DeviceThread_ConfigureVideo(InCommandList);

			FrameRate = Helpers::ConvertToUnrealFramerate(DesiredVideoFormat);

			if (VideoBufferSize > 0)
			{
				AJAAutoLock AutoLock(&FrameLock);
				for (Frame* IttFrame : AllFrames)
				{
					AJA_CHECK(IttFrame->VideoBuffer == nullptr);
#if AJA_TEST_MEMORY_BUFFER
					IttFrame->VideoBuffer = (uint8_t*)AJAMemory::AllocateAligned(VideoBufferSize + 1, AJA_PAGE_SIZE);
					IttFrame->VideoBuffer[VideoBufferSize] = AJA_TEST_MEMORY_END_TAG;
#else
					IttFrame->VideoBuffer = (uint8_t*)AJAMemory::AllocateAligned(VideoBufferSize, AJA_PAGE_SIZE);
#endif
				}
			}

			return bResult;
		}

		void OutputChannelThread::Thread_PushAvailableReadingFrame(Frame* InCurrentReadingFrame)
		{
			// reset the buffer size for the next write
			AJAAutoLock AutoLock(&FrameLock);
			InCurrentReadingFrame->Clear();
			FrameReadyToWrite.push_back(InCurrentReadingFrame);
		}

		bool OutputChannelThread::DeviceThread_ConfigureAutoCirculate(DeviceConnection::CommandList& InCommandList)
		{
			const int32_t NumberOfLinkChannel = Helpers::GetNumberOfLinkChannel(GetOptions().TransportType);
			for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
			{
				const NTV2Channel ChannelItt = NTV2Channel(int32_t(Channel) + ChannelIndex);
				AJA_CHECK(GetDevice().AutoCirculateStop(ChannelItt));
			}

			{
				// NB. We are already lock
				ULWord OptionFlags = (UseTimecode() ? AUTOCIRCULATE_WITH_RP188 : 0);
				OptionFlags |= (UseAncillary() || UseAncillaryField2() ? AUTOCIRCULATE_WITH_ANC : 0);
				UByte FrameCount = 0;
				UByte FirstFrame = BaseFrameIndex;
				UByte EndFrame = FirstFrame + DeviceConnection::NumberOfFrameForAutoCirculate - 1;

				AJA_CHECK(GetDevice().AutoCirculateInitForOutput(Channel, FrameCount, AudioSystem, OptionFlags, 1, FirstFrame, EndFrame));
			}
			AJA_CHECK(GetDevice().AutoCirculateStart(Channel));

			AJA_CHECK(Device->WaitForInputOrOutputInterrupt(Channel));

			return true;
		}

		void OutputChannelThread::Thread_AutoCirculateLoop()
		{
			const bool bIsProgressive = ::IsProgressivePicture(VideoFormat) || Options.bOutputInterlaceAsProgressive;
			bool bRunning = true;
			bool bSleep = true;

			AUTOCIRCULATE_TRANSFER Transfer;
			while (bRunning && !bStopRequested)
			{
				bool bIsValidWaitForOutput = true;
				if (bSleep)
				{
					// I would expect to sync here, but it seems that causes an extra frame
					// of latency, so sleep, until the next frame is ready instead.
					::Sleep(0);
				}
				else
				{
					if (bIsProgressive)
					{
						bRunning = Device->WaitForInputOrOutputInterrupt(Channel);
					}
					else
					{
						// In Interlaced, wait for Field0 to continue but warn the user when field is 1 (in case he wants to be v-sync)
						NTV2FieldID FieldId = NTV2_FIELD_INVALID;
						bRunning = Device->WaitForInputOrOutputInputField(Channel, 1, FieldId);
						bIsValidWaitForOutput = (bRunning && FieldId == NTV2_FIELD0);
					}
				}

				if (!bRunning)
				{
					UE_LOGF(LogAjaCore, Error, "AutoCirculate: Can't wait for the output field for channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
					if (GetOptions().bStopOnTimeout)
					{
						break;
					}
					else
					{
						bRunning = true;
					}
				}

				AUTOCIRCULATE_STATUS ChannelStatus;
				bRunning = GetDevice().AutoCirculateGetStatus(Channel, ChannelStatus);
				if (!bRunning)
				{
					UE_LOGF(LogAjaCore, Error, "AutoCirculate: Can't get the status for output channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
					break;
				}

				if (bStopRequested)
				{
					// After the WaitForInputOrOutputInterrupt, it's possible that bStopRequested has been modified.
					bRunning = false;
					break;
				}

				if (!bIsValidWaitForOutput)
				{
					AJAAutoLock AutoLock(&Lock);
					if (GetOptions().CallbackInterface)
					{
						GetOptions().CallbackInterface->OnOutputFrameStarted();
					}
					continue;
				}

				bSleep = true;
				if (ChannelStatus.CanAcceptMoreOutputFrames())
				{
					bSleep = false;
					{
						AJAAutoLock AutoLock(&Lock);
						if (GetOptions().CallbackInterface)
						{
							GetOptions().CallbackInterface->OnOutputFrameStarted();
						}
					}

					Frame* AvailableReadingFrame = Thread_FetchAvailableReadingFrame();
					if (AvailableReadingFrame)
					{
						Thread_TestInterlacedOutput(AvailableReadingFrame);


						AJAOutputFrameData FrameData;
						FrameData.FramesDropped = ChannelStatus.acFramesDropped;
						FrameData.FramesLost = LostFrameCounter;

						bSleep = false; 

						if ((UseAncillary() && AvailableReadingFrame->CopiedAncBufferSize) || (UseAncillaryField2() && AvailableReadingFrame->CopiedAncF2BufferSize))
						{
							ULWord* Ancillary = UseAncillary() ? reinterpret_cast<ULWord*>(AvailableReadingFrame->AncBuffer) : nullptr;
							ULWord AncillarySize = UseAncillary() ? AvailableReadingFrame->CopiedAncBufferSize : 0;
							ULWord* AncillaryF2 = UseAncillaryField2() ? reinterpret_cast<ULWord*>(AvailableReadingFrame->AncF2Buffer) : nullptr;
							ULWord AncillaryF2Size = UseAncillaryField2() ? AvailableReadingFrame->CopiedAncF2BufferSize : 0;
							Transfer.SetAncBuffers(Ancillary, AncBufferSize, AncillaryF2, AncF2BufferSize);
						}

						if (UseAudio() && AvailableReadingFrame->CopiedAudioBufferSize)
						{
							Transfer.SetAudioBuffer(reinterpret_cast<ULWord*>(AvailableReadingFrame->AudioBuffer), AvailableReadingFrame->CopiedAudioBufferSize);
						}

						if (UseVideo() && AvailableReadingFrame->CopiedVideoBufferSize && TextureTransfer && GetOptions().bUseGPUDMA)
						{
							const UE::GPUTextureTransfer::ETransferDirection Direction = UE::GPUTextureTransfer::ETransferDirection::GPU_TO_CPU;
							TextureTransfer->BeginSync(AvailableReadingFrame->VideoBuffer, Direction);
						}

						if (UseVideo() && AvailableReadingFrame->CopiedVideoBufferSize)
						{
							if (bIsProgressive)
							{
								BurnTimecode(AvailableReadingFrame->Timecode, AvailableReadingFrame->VideoBuffer);
							}
							else
							{
								BurnTimecode(AvailableReadingFrame->Timecode, AvailableReadingFrame->Timecode2, AvailableReadingFrame->VideoBuffer);
							}
							Transfer.SetVideoBuffer(reinterpret_cast<ULWord*>(AvailableReadingFrame->VideoBuffer), AvailableReadingFrame->CopiedVideoBufferSize);
						}

						if (UseTimecode())
						{
							FrameData.Timecode = AvailableReadingFrame->Timecode;
							const FTimecode ConvertedTimecode = Helpers::AdjustTimecodeFromUE(VideoFormat, AvailableReadingFrame->Timecode);
							NTV2_RP188 Timecode = Helpers::ConvertTimecodeToRP188(ConvertedTimecode);
							Transfer.SetOutputTimeCode(Timecode, NTV2ChannelToTimecodeIndex(Channel)); //Set timecode on LTC and VITC. Do not use SetAllOutputtimecode. It trashes timecodes of other pins
						}

						AJA_CHECK(GetDevice().AutoCirculateTransfer(Channel, Transfer));
						if (UseVideo() && AvailableReadingFrame->CopiedVideoBufferSize && TextureTransfer && GetOptions().bUseGPUDMA)
						{
							TextureTransfer->EndSync(AvailableReadingFrame->VideoBuffer);
						}

						Thread_PushAvailableReadingFrame(AvailableReadingFrame);

						{
							AJAAutoLock AutoLock(&Lock);
							if (GetOptions().CallbackInterface)
							{
								bRunning = GetOptions().CallbackInterface->OnOutputFrameCopied(FrameData);
							}
						}

						if (!bRunning)
						{
							break;
						}
					}
				}
			}

			AJA_CHECK(GetDevice().AutoCirculateStop(Channel));

			//	Write the requested test pattern into host buffer...
			{
				NTV2TestPatternGen testPatternGen;
				NTV2Buffer testPatternBuffer;

				testPatternGen.DrawTestPattern(NTV2TestPatternSelect::NTV2_TestPatt_ColorBars100,
					FormatDescriptor,
					testPatternBuffer);

				for (UByte FrameIndex = BaseFrameIndex; FrameIndex < BaseFrameIndex + DeviceConnection::NumberOfFrameForAutoCirculate; ++FrameIndex)
				{
					GetDevice().DMAWriteFrame(FrameIndex, reinterpret_cast <uint32_t *> (testPatternBuffer.GetHostPointer()), uint32_t(testPatternBuffer.GetByteCount()));
				}
			}

			if (!bStopRequested)
			{
				AJAAutoLock AutoLock(&Lock);
				if (GetOptions().CallbackInterface)
				{
					GetOptions().CallbackInterface->OnCompletion(bRunning);
				}
			}
		}

		bool OutputChannelThread::DeviceThread_ConfigurePingPong(DeviceConnection::CommandList& InCommandList)
		{
			Frame* CurrentFrame = nullptr;
			{
				AJAAutoLock AutoLock(&FrameLock);
				if (AllFrames.size() == 0)
				{
					return false;
				}
				CurrentFrame = AllFrames.front();
			}

			ThreadLock_PingPongOutputLoop_Memset0(CurrentFrame);
		
			Device->WaitForInputOrOutputInterrupt(Channel);

			if (bTraceDebugMarkers)
			{
				InterruptEventHandle = Device->SetOnInterruptEvent(Channel, DeviceConnection::FOnVerticalInterrupt::FDelegate::CreateRaw(this, &OutputChannelThread::OnVerticalInterrupt));
			}

			if (CVarAjaPreRollMethod.GetValueOnAnyThread() == 0)
			{
				// Clear the current Buffer and initialize the ping-pong
				const int32_t NumOutputFrameIndex = 2;
				for (int32_t Index = 0; Index < NumOutputFrameIndex; ++Index)
				{
					AJA_CHECK(GetDevice().SetOutputFrame(Channel, BaseFrameIndex + Index));

					//@TODO fix ancillary for pingpong. AJA told us they would help
					if (UseAncillary())
					{
						NTV2_POINTER AncBufferPointer(CurrentFrame->AncBuffer, CurrentFrame->CopiedAncBufferSize);
						GetDevice().DMAWriteAnc(Index, AncBufferPointer);
					}
					if (UseAncillaryField2())
					{
						NTV2_POINTER AncF2BufferPointer(CurrentFrame->AncF2Buffer, CurrentFrame->CopiedAncF2BufferSize);
						GetDevice().DMAWriteAnc(Index, AncF2BufferPointer);
					}
					if (UseAudio())
					{
						GetDevice().DMAWriteAudio(AudioSystem, reinterpret_cast<ULWord*>(CurrentFrame->AudioBuffer), 0, CurrentFrame->CopiedAudioBufferSize);
						CurrentAudioWriteOffset += CurrentFrame->CopiedAudioBufferSize;
					}
					if (UseVideo())
					{
						AJA_CHECK(GetDevice().DMAWriteFrame(BaseFrameIndex + Index, reinterpret_cast<ULWord*>(CurrentFrame->VideoBuffer), CurrentFrame->CopiedVideoBufferSize, Channel));

						if (TextureTransfer && GetOptions().bUseGPUDMA)
						{
							TextureTransfer->EndSync(CurrentFrame->VideoBuffer);
						}
					}
					Device->WaitForInputOrOutputInterrupt(Channel);
				}

				ULWord VBICount = 0;
				AJA_CHECK(GetDevice().GetOutputVerticalInterruptCount(VBICount, Channel));

				// SetOutputFrame will tell the card that a frame should be transferred starting on the next VBI, so it will be transferring starting at CurrentVBI + 1, and be done transferring at CurrentVBI + 2
				BufferFreeIndex[0] = VBICount + 2;
				BufferFreeIndex[1] = BufferFreeIndex[0] + 1;

				//	Before the main loop starts, ping-pong the buffers so the hardware will use
				//	different buffers than the ones it was using while idling...
				uint32_t CurrentOutFrame = 0;
				CurrentOutFrame ^= 1;
				AJA_CHECK(GetDevice().SetOutputFrame(Channel, BaseFrameIndex + CurrentOutFrame));
			}

			return true;
		}

		void OutputChannelThread::ThreadLock_PingPongOutputLoop_Memset0(Frame* InCurrentReadingFrame)
		{
			if (InCurrentReadingFrame->AncBuffer)
			{
				AJA_CHECK(UseAncillary());
				memset(InCurrentReadingFrame->AncBuffer, 0, AncBufferSize);
			}
			if (InCurrentReadingFrame->AncF2Buffer)
			{
				AJA_CHECK(UseAncillaryField2());
				memset(InCurrentReadingFrame->AncF2Buffer, 0, AncF2BufferSize);
			}
			if (InCurrentReadingFrame->AudioBuffer)
			{
				AJA_CHECK(UseAudio());
				memset(InCurrentReadingFrame->AudioBuffer, 0, AudioBufferSize);
			}
			if (InCurrentReadingFrame->VideoBuffer)
			{
				AJA_CHECK(UseVideo());
				memset(InCurrentReadingFrame->VideoBuffer, 0, VideoBufferSize);
			}
		}

		// As reference: this code was inspired by NTV2LLBurn::ProcessFrames(void)
		void OutputChannelThread::Thread_PingPongLoop()
		{
			bool bHaveOutputOnce = false;
			const bool bIsProgressive = ::IsProgressivePicture(VideoFormat) || Options.bOutputInterlaceAsProgressive;

			uint32_t CurrentOutFrame = 0;
			// If prerolling normally, then we're targetting the free buffer.
			CurrentOutFrame ^= 1;

			if (TextureTransfer)
			{
				TextureTransfer->ThreadPrep();
			}

			uint32_t LastFrameDropCount = 0;
			auto LogDropFrames = [&LastFrameDropCount, this]()
			{
				UE_LOGF(LogAjaCore, Warning, "PingPong: Dropped %d frames while outputting on channel %d on device %s.\n", LastFrameDropCount, uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
			};

			bool bRunning = true;
			while (bRunning && !bStopRequested)
			{
				if (bOutputWithReduceWaiting)
				{
					const double MaxWaitTime = FrameRate.AsInterval();
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(AjaOutputChannelThread::WaitingForAvailableFrame);
						if (!FrameAvailableEvent->Wait(MaxWaitTime * 1000))
						{
							LastFrameDropCount++;
							continue;
						}
					}
				}

				//	Wait until the input has completed capturing a frame...
				bool bIsValidWaitForOutput = true;
				if (bIsProgressive)
				{
					if (bOutputWithReduceWaiting)
					{
						ULWord VBICount = 0;
						AJA_CHECK(GetDevice().GetOutputVerticalInterruptCount(VBICount, Channel));

						// If we're still using the card buffer, wait until next VBI, otherwise go ahead.
						const uint32 OffScreenBuffer = CurrentOutFrame ^ 1;

						if (VBICount + 2 == BufferFreeIndex[CurrentOutFrame])
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(AjaOutputChannelThread::DroppedFrame);
							// We have already scheduled an output in the same VBI period, this usually occurs after a hitch, so we should drop one of our frames 
							if (Frame* AvailableReadingFrame = Thread_FetchAvailableReadingFrame())
							{
								Thread_PushAvailableReadingFrame(AvailableReadingFrame);
							}

							continue;
						}
						else if (VBICount < BufferFreeIndex[OffScreenBuffer])
						{
							// Off screen buffer is not available, so wait until a VBI occurs so we know it's free.
							TRACE_CPUPROFILER_EVENT_SCOPE(AjaOutputChannelThread::WaitForInputOrOutputInterrupt);
							bRunning = Device->WaitForInputOrOutputInterrupt(Channel);
						}
					}
					else
					{
						bRunning = Device->WaitForInputOrOutputInterrupt(Channel);
					}
				}
				else
				{
					// In Interlaced, wait for Field0 to continue but warn the user when field is 1 (in case he wants to be v-sync)
					NTV2FieldID FieldId = NTV2_FIELD_INVALID;
					bRunning = Device->WaitForInputOrOutputInputField(Channel, 1, FieldId);
					bIsValidWaitForOutput = (bRunning && FieldId == NTV2_FIELD0);
				}

				FScopeLock DeviceLock(&DeviceCriticalSection);

				if (!bRunning)
				{
					UE_LOGF(LogAjaCore, Error, "PingPong: Can't wait for the output field for channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
					if (GetOptions().bStopOnTimeout)
					{
						break;
					}
					else
					{
						bRunning = true;
					}
				}

				if (bStopRequested)
				{
					// After the WaitForInputOrOutputInterrupt, it's possible that bStopRequested has been modified.
					bRunning = false;
					break;
				}

				{
					AJAAutoLock AutoLock(&Lock);
					if (GetOptions().CallbackInterface)
					{
						GetOptions().CallbackInterface->OnOutputFrameStarted();
					}
				}

				if (!bIsValidWaitForOutput)
				{
					continue;
				}

				Frame* AvailableReadingFrame = Thread_FetchAvailableReadingFrame();
				if (AvailableReadingFrame)
				{
					if (LastFrameDropCount != 0)
					{
						LogDropFrames();
						LastFrameDropCount = 0;
					}

					Thread_TestInterlacedOutput(AvailableReadingFrame);

					//	Flip sense of the buffers again to refer to the buffers that the hardware isn't using (i.e. the off-screen buffers)...
					CurrentOutFrame ^= 1;

					//	Check for dropped frames by ensuring the hardware has not started to process
					//	the buffers that were just filled....
					uint32_t ReadBackIn;
					AJA_CHECK(GetDevice().GetOutputFrame(Channel, ReadBackIn));
					if (ReadBackIn == BaseFrameIndex + CurrentOutFrame && bHaveOutputOnce)
					{
						++PingPongDropCount;
					}

					AJAOutputFrameData FrameData;
					FrameData.FramesDropped = PingPongDropCount;
					FrameData.FramesLost = LostFrameCounter;

					if (AvailableReadingFrame->CopiedAncBufferSize > 0)
					{
						NTV2_POINTER AncBufferPointer(AvailableReadingFrame->AncBuffer, AvailableReadingFrame->CopiedAncBufferSize);
						bRunning = bRunning && GetDevice().DMAWriteAnc(CurrentOutFrame, AncBufferPointer);
					}

					if (AvailableReadingFrame->CopiedAncF2BufferSize > 0)
					{
						NTV2_POINTER AncF2BufferPointer(AvailableReadingFrame->AncF2Buffer, AvailableReadingFrame->CopiedAncF2BufferSize);
						bRunning = bRunning && GetDevice().DMAWriteAnc(CurrentOutFrame, AncF2BufferPointer);
					}

					ULWord VBICount = 0;
					AJA_CHECK(GetDevice().GetOutputVerticalInterruptCount(VBICount, Channel));


					if (AvailableReadingFrame->CopiedVideoBufferSize > 0)
					{
						if (bIsProgressive)
						{
							BurnTimecode(AvailableReadingFrame->Timecode, AvailableReadingFrame->VideoBuffer);
						}
						else
						{
							BurnTimecode(AvailableReadingFrame->Timecode, AvailableReadingFrame->Timecode2, AvailableReadingFrame->VideoBuffer);
						}

						if (TextureTransfer && GetOptions().bUseGPUDMA)
						{
							const UE::GPUTextureTransfer::ETransferDirection Direction = UE::GPUTextureTransfer::ETransferDirection::GPU_TO_CPU;
							TextureTransfer->BeginSync(AvailableReadingFrame->VideoBuffer, Direction);
						}

						{
							TRACE_CPUPROFILER_EVENT_SCOPE(AjaOutputChannelThread::DMAWriteFrame);
							bRunning = bRunning && GetDevice().DMAWriteFrame(BaseFrameIndex + CurrentOutFrame, reinterpret_cast<ULWord*>(AvailableReadingFrame->VideoBuffer), AvailableReadingFrame->CopiedVideoBufferSize, Channel);
						}

						if (TextureTransfer && GetOptions().bUseGPUDMA)
						{
							TextureTransfer->EndSync(AvailableReadingFrame->VideoBuffer);
						}
					}

					if (UseAudio() && AvailableReadingFrame->CopiedAudioBufferSize && !GetOptions().bDirectlyWriteAudio)
					{
						bRunning &= Thread_HandleAudio(TEXT("PingPong"), AvailableReadingFrame);
					}

					if (!bRunning)
					{
						UE_LOGF(LogAjaCore, Error, "PingPong: Can't do the DMA frame transfer for channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
						break;
					}

					//	Determine the for the DMA timecode value
					if (UseTimecode())
					{
						FrameData.Timecode = AvailableReadingFrame->Timecode;
						// As reference: this code was inspirered by NTV2LLBurn::InputSignalHasTimecode
						{
							const FTimecode ConvertedTimecode = Helpers::AdjustTimecodeFromUE(VideoFormat, AvailableReadingFrame->Timecode);
							NTV2_RP188 Timecode = Helpers::ConvertTimecodeToRP188(ConvertedTimecode);

							const int32_t NumberOfLinkChannel = Helpers::GetNumberOfLinkChannel(GetOptions().TransportType);
							{
								for (int32_t ChannelIndex = 0; ChannelIndex < NumberOfLinkChannel; ++ChannelIndex)
								{
									const NTV2Channel ChannelItt = NTV2Channel(int32_t(Channel) + ChannelIndex);
									AJA_CHECK(GetDevice().SetRP188Data(ChannelItt, Timecode));
								}
							}
						}
					}

					//	Tell the hardware which buffers to start using at the beginning of the next frame...
					if (bTraceDebugMarkers)
					{
						TRACE_BOOKMARK(TEXT("Aja: SetOutputFrame [%d] (Buffer %d)"), AvailableReadingFrame->FrameIdentifier, CurrentOutFrame);
					}

					AJA_CHECK(GetDevice().SetOutputFrame(Channel, BaseFrameIndex + CurrentOutFrame));

					if (!bRunning)
					{
						UE_LOGF(LogAjaCore, Error, "PingPong: Can't do the DMA frame transfer for channel %d on device %s.\n", uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());
						Thread_PushAvailableReadingFrame(AvailableReadingFrame);
						break;
					}

					// SetOutputFrame will tell the card that a frame should be transferred starting on the next VBI, so it will be transferring starting at CurrentVBI + 1, and be done transferring at CurrentVBI + 2
					uint32 FrameAvailableVBICount = VBICount + 2;
					BufferFreeIndex[CurrentOutFrame] = FrameAvailableVBICount;
					BufferFrameIdMapping[CurrentOutFrame] = AvailableReadingFrame->FrameIdentifier;

					Thread_PushAvailableReadingFrame(AvailableReadingFrame);
					bHaveOutputOnce = true;

					{
						AJAAutoLock AutoLock(&Lock);
						if (GetOptions().CallbackInterface)
						{
							bRunning = GetOptions().CallbackInterface->OnOutputFrameCopied(FrameData);
						}
					}
				}
				else if (GetOptions().bDisplayWarningIfDropFrames)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(OutputChannel::NoFramesAvailable);

					if (UseAudio() && !GetOptions().bDirectlyWriteAudio)
					{
						Thread_HandleLostFrameAudio();
					}

					static constexpr uint32 NumMaxFrameBeforeWarning = 50;

					LastFrameDropCount++;
					if (LastFrameDropCount > NumMaxFrameBeforeWarning)
					{
						LogDropFrames();
						LastFrameDropCount = 0;
					}
				}
			}

			//	Write the requested test pattern into host buffer...
			{
				NTV2TestPatternGen testPatternGen;
				NTV2Buffer testPatternBuffer;

				testPatternGen.DrawTestPattern(NTV2TestPatternSelect::NTV2_TestPatt_ColorBars100,
					FormatDescriptor,
					testPatternBuffer);

				CurrentOutFrame ^= 1;
				GetDevice().DMAWriteFrame(BaseFrameIndex + CurrentOutFrame, reinterpret_cast <uint32_t*> (testPatternBuffer.GetHostPointer()), uint32_t(testPatternBuffer.GetByteCount()));
				CurrentOutFrame ^= 1;
				GetDevice().DMAWriteFrame(BaseFrameIndex + CurrentOutFrame, reinterpret_cast <uint32_t*> (testPatternBuffer.GetHostPointer()), uint32_t(testPatternBuffer.GetByteCount()));
			}

			if (!bStopRequested)
			{
				AJAAutoLock AutoLock(&Lock);
				if (GetOptions().CallbackInterface)
				{
					GetOptions().CallbackInterface->OnCompletion(bRunning);
				}
			}

			if (TextureTransfer)
			{
				TextureTransfer->ThreadCleanup();
			}
		}

		void OutputChannelThread::Thread_TestInterlacedOutput(Frame* InFrame)
		{
			if (!GetOptions().bTEST_OutputInterlaced)
			{
				return;
			}

			struct FUYV422Format
			{
				uint8_t V;
				uint8_t Y1;
				uint8_t U;
				uint8_t Y0;
			};
			auto IsLineWhite = [](const FUYV422Format* pLine) -> bool { return pLine->Y0 >= 0x0 && pLine->Y0 < 0x20 && pLine->Y1 >= 0x0 && pLine->Y1 < 0x20; };
			const FUYV422Format* pFirstLine = reinterpret_cast<const FUYV422Format*>(InFrame->VideoBuffer);
			const FUYV422Format* pSecondLine = pFirstLine + FormatDescriptor.GetBytesPerRow()/sizeof(FUYV422Format);

			if (InterlacedTest_FrameCounter == 0)
			{
				bInterlacedTest_ExpectFirstLineToBeWhite = IsLineWhite(pFirstLine);
			}

			bool bIsFirstLineWhite = IsLineWhite(pFirstLine);
			bool pIsSecondLineWhite = IsLineWhite(pSecondLine);
			if (bIsFirstLineWhite == pIsSecondLineWhite)
			{
				UE_LOGF(LogAjaCore, Error, "INTERLACED TEST - The 2 lines are the same color. %d\n", InterlacedTest_FrameCounter);
			}
			if (bIsFirstLineWhite != bInterlacedTest_ExpectFirstLineToBeWhite)
			{
				bInterlacedTest_ExpectFirstLineToBeWhite = !bInterlacedTest_ExpectFirstLineToBeWhite;
				UE_LOGF(LogAjaCore, Error, "INTERLACED TEST - The lines has swap color. %d\n", InterlacedTest_FrameCounter);
			}
			++InterlacedTest_FrameCounter;
		}

		bool OutputChannelThread::Thread_GetAudioOffset(int32& OutAudioOffset)
		{
			bool bSuccess = true;
			ULWord AudioWrapOffset = 0;
			ULWord PlayHeadPosition = 0;

			bSuccess &= GetDevice().GetAudioWrapAddress(AudioWrapOffset);
			bSuccess &= GetDevice().ReadAudioLastOut(PlayHeadPosition);

			// Calculate audio offset
			if (CurrentAudioWriteOffset > PlayHeadPosition)
			{
				OutAudioOffset = CurrentAudioWriteOffset - PlayHeadPosition;
			}
			else
			{
				OutAudioOffset = (AudioWrapOffset - PlayHeadPosition) + CurrentAudioWriteOffset;
			}

			return bSuccess;
		}


		bool OutputChannelThread::Thread_GetAudioOffsetInSeconds(double& OutAudioOffset)
		{
			int32 OffsetInBytes = 0;
			if (Thread_GetAudioOffset(OffsetInBytes))
			{
				OutAudioOffset = (OffsetInBytes / sizeof(int32)) / 48000.f / Options.NumberOfAudioChannel;
				return true;
			}

			return false;
		}

		bool OutputChannelThread::Thread_GetAudioOffsetInSamples(int32& OutAudioOffset)
		{
			int32 OffsetInBytes = 0;
			if (Thread_GetAudioOffset(OffsetInBytes))
			{
				OutAudioOffset = FMath::RoundToInt32((float)OffsetInBytes / Options.NumberOfAudioChannel / sizeof(int32));
				return true;
			}

			return false;
		}


		bool OutputChannelThread::Thread_TransferAudioBuffer(const uint8* InBuffer, int32 InBufferSize)
		{
			bool bSuccess = true;

			ULWord AudioWrapOffset = 0;

			bSuccess &= GetDevice().GetAudioWrapAddress(AudioWrapOffset);

			if (CurrentAudioWriteOffset + InBufferSize < AudioWrapOffset)
			{
				// Simplest case, just write the whole data to the SDRAM.
				bSuccess &= GetDevice().DMAWriteAudio(AudioSystem, reinterpret_cast<const ULWord*>(InBuffer), CurrentAudioWriteOffset, InBufferSize);
				CurrentAudioWriteOffset += InBufferSize;
			}
			else
			{
				// Audio data will wrap, so write in two parts.
				ULWord BytesUntilBufferEnd = AudioWrapOffset - CurrentAudioWriteOffset;
				LWord RemainingBytes = InBufferSize - BytesUntilBufferEnd;
				// We might technically have space remaining, but we can't write there since it's too close to the wrap address.
				if (RemainingBytes < 0)
				{
					RemainingBytes = 0;
				}

				bSuccess &= GetDevice().DMAWriteAudio(AudioSystem, reinterpret_cast<const ULWord*>(InBuffer), CurrentAudioWriteOffset, BytesUntilBufferEnd);
				if (RemainingBytes > 0)
				{
					bSuccess &= GetDevice().DMAWriteAudio(AudioSystem, reinterpret_cast<const ULWord*>(InBuffer + BytesUntilBufferEnd), 0, RemainingBytes);
				}

				CurrentAudioWriteOffset = RemainingBytes;
			}

			return bSuccess;
		}

		bool OutputChannelThread::Thread_HandleAudio(const FString& OutputMethod, Frame* AvailableReadingFrame)
		{
			bool bAudioTransferWasSuccessful = true;

			bool bAudioSuccess = true;

			bool bAudioOutputRunning = false;
			bAudioSuccess &= GetDevice().IsAudioOutputRunning(AudioSystem, bAudioOutputRunning);

			bool bIsPaused = false;
			bAudioSuccess &= GetDevice().GetAudioOutputPause(AudioSystem, bIsPaused);

			if (bIsPaused && bAudioOutputRunning && !Thread_ShouldPauseAudioOutput())
			{
				UE_LOGF(LogAjaCore, Verbose, "%ls: Resuming audio output.", *OutputMethod);
				bool bPausePlayback = false;
				bAudioSuccess &= GetDevice().SetAudioOutputPause(AudioSystem, bPausePlayback);
				bIsPaused = true;
			}
			else if (!bAudioOutputRunning)
			{
				// Only offset the start ptr if this is1 the first time we're starting the audio output.

				ULWord WrapOffset = 0;
				bAudioSuccess &= GetDevice().GetAudioWrapAddress(WrapOffset);

				int32 NumOffsetFrames = 1;
				int32 InitialOffset = Options.NumberOfAudioChannel * NumSamplesPerFrame * NumOffsetFrames * sizeof(int32);

				CurrentAudioWriteOffset = InitialOffset;

				UE_LOGF(LogAjaCore, Verbose, "%ls: Starting audio output.", *OutputMethod);
				bAudioSuccess &= GetDevice().StartAudioOutput(AudioSystem); // This resets the playhead to 0, we have to override that to something closer to the write head in order to reduce audio delay
			}

			if (bAudioSuccess)
			{
				double OffsetInSeconds = 0;
				bAudioSuccess &= Thread_GetAudioOffsetInSeconds(OffsetInSeconds);
				ULWord PlayHeadPosition = 0;
				bAudioSuccess &= GetDevice().ReadAudioLastOut(PlayHeadPosition, AudioSystem);
				AudioPlayheadLastPosition = PlayHeadPosition;
				
				SET_FLOAT_STAT(STAT_AjaMediaCapture_Audio_Delay, OffsetInSeconds);
				UE_LOGF(LogAjaCore, Verbose, "%ls: Play head: %d, Write Head: %d, Offset: %f", *OutputMethod, AudioPlayheadLastPosition.load(), CurrentAudioWriteOffset.load(), OffsetInSeconds);

				bAudioTransferWasSuccessful &= Thread_TransferAudioBuffer(AvailableReadingFrame->AudioBuffer, AvailableReadingFrame->CopiedAudioBufferSize);

				int32 OffsetInSamples = 0;
				bAudioSuccess &= Thread_GetAudioOffsetInSamples(OffsetInSamples);

				if (bAudioSuccess && Thread_ShouldPauseAudioOutput())
				{
					UE_LOGF(LogAjaCore, Warning, "%ls: Audio play head is catching up to write head for channel %d on device %s.\n", *OutputMethod, uint32_t(Channel) + 1, GetDevice().GetDisplayName().c_str());

					bool bPausePlayout = true;
					GetDevice().SetAudioOutputPause(AudioSystem, bPausePlayout);

					UE_LOGF(LogAjaCore, Verbose, "%ls Pausing audio output because the play head is catching up to the write head. Last playhead position: %d", *OutputMethod, AudioPlayheadLastPosition.load());
				}
			}
			else
			{
				UE_LOGF(LogAjaCore, Verbose, "%ls Audio output failed.", *OutputMethod);
			}

			return bAudioTransferWasSuccessful;
		}

		bool OutputChannelThread::DMAWriteAudio(const uint8_t* InAudioBuffer, int32_t BufferSize)
		{
			if (!Device || !Device->GetCard())
			{
				return false;
			}

			TRACE_CPUPROFILER_EVENT_SCOPE(OutputChannelThread::DMAWriteAudio);
			FScopeLock DeviceLock(&DeviceCriticalSection);

			bool bAudioSuccess = true;

			bool bAudioOutputRunning = false;
			bAudioSuccess &= GetDevice().IsAudioOutputRunning(AudioSystem, bAudioOutputRunning);

			UE_CLOGF(!bAudioSuccess, LogAjaCore, Verbose, "IsAudioOutputRunning returned false.");

			bool bIsPaused = false;
			bAudioSuccess &= GetDevice().GetAudioOutputPause(AudioSystem, bIsPaused);

			UE_CLOGF(!bAudioSuccess, LogAjaCore, Verbose, "GetAudioOutputPause returned false.");

			// Start audio output as late as possible to reduce delay.
			if (!bAudioOutputRunning)
			{
				// Only offset the start ptr if this is the first time we're starting the audio output.
				ULWord WrapOffset = 0;
				bAudioSuccess &= GetDevice().GetAudioWrapAddress(WrapOffset);

				constexpr int32 NumOffsetFrames = 4;
				const int32 InitialOffset = Options.NumberOfAudioChannel * NumSamplesPerFrame * NumOffsetFrames * sizeof(int32);

				CurrentAudioWriteOffset = InitialOffset;

				UE_LOGF(LogAjaCore, Verbose, "PingPong: Starting audio output.");
				bAudioSuccess &= GetDevice().StartAudioOutput(AudioSystem); // This resets the playhead to 0, we have to override that to something closer to the write head in order to reduce audio delay
			}

			double OffsetInSeconds = 0;
			bAudioSuccess &= Thread_GetAudioOffsetInSeconds(OffsetInSeconds);
			UE_CLOGF(!bAudioSuccess, LogAjaCore, Verbose, "Thread_GetAudioOffsetInSeconds returned false.");
			ULWord PlayHeadPosition = 0;
			bAudioSuccess &= GetDevice().ReadAudioLastOut(PlayHeadPosition, AudioSystem);
			UE_CLOGF(!bAudioSuccess, LogAjaCore, Verbose, "ReadAudioLastOut returned false.");
			
			AudioPlayheadLastPosition = PlayHeadPosition;
			SET_FLOAT_STAT(STAT_AjaMediaCapture_Audio_Delay, OffsetInSeconds);
			UE_LOGF(LogAjaCore, Verbose, "%ls: Play head: %d, Write Head: %d, Offset: %f", TEXT("PingPong"), AudioPlayheadLastPosition.load(), CurrentAudioWriteOffset.load(), OffsetInSeconds);

			bAudioSuccess &= Thread_TransferAudioBuffer(InAudioBuffer, BufferSize);
			
			return bAudioSuccess;
		}
		

		bool OutputChannelThread::Thread_HandleLostFrameAudio()
		{
			bool bAudioSuccess = true;
			bool bAudioOutputRunning = false;
			bAudioSuccess &= GetDevice().IsAudioOutputRunning(AudioSystem, bAudioOutputRunning);

			if (bAudioOutputRunning)
			{
				// When dropping frames, we have to keep writing audio data or else the play head will catch up to the write head which will induce a delay
				int32 OffsetInSeconds = 0;
				bAudioSuccess &= Thread_GetAudioOffsetInSamples(OffsetInSeconds);

				if (bAudioSuccess && Thread_ShouldPauseAudioOutput())
				{
					constexpr bool bPausePlayout = true;
					bAudioSuccess &= GetDevice().SetAudioOutputPause(AudioSystem, bPausePlayout);
					UE_LOGF(LogAjaCore, Warning, "Pausing audio output. Last playhead position: %d", AudioPlayheadLastPosition.load());
				}
			}

			return bAudioSuccess;
		}

		bool OutputChannelThread::Thread_ShouldPauseAudioOutput()
		{
			constexpr int32 AudioBufferZoneInFrames = 1;
			int32 OffsetInSamples = 0;
			Thread_GetAudioOffsetInSamples(OffsetInSamples);

			if (OffsetInSamples <= (int32) NumSamplesPerFrame * AudioBufferZoneInFrames)
			{
				return true;
			}
			return false;
		}

		void OutputChannelThread::OnVerticalInterrupt(uint32 SyncCount) const
		{
			uint32 FrameId = 0;
			uint8 BufferId = 0;

			if (BufferFreeIndex[0] == SyncCount)
			{
				FrameId = BufferFrameIdMapping[0];
				BufferId = 0;
			}
			else if (BufferFreeIndex[1] == SyncCount)
			{
				FrameId = BufferFrameIdMapping[1];
				BufferId = 1;
			}

			if (FrameId != 0)
			{
				TRACE_BOOKMARK(TEXT("Aja: Finish SDI transfer frame [%d] (Buffer: %d)"), FrameId, BufferId);
			}
			else if (BufferFrameIdMapping[0] != 0 && BufferFrameIdMapping[1] != 0) // Don't log drop frames when starting the output
			{
				TRACE_BOOKMARK(TEXT("Aja VBI [%d] (Dropped frame)"), SyncCount);
			}
		}
	}
}