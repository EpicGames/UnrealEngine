// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChannelBase.h"
#include "GPUTextureTransferModule.h"
#include "Misc/Timecode.h"

#include <vector>

class FRHITexture;

namespace AJA
{
	namespace Private
	{
		class OutputChannelThread;


		/* OutputChannel definition
		*****************************************************************************/
		class OutputChannel
		{
		public:
			OutputChannel() = default;
			OutputChannel(const OutputChannel&) = delete;
			OutputChannel& operator=(const OutputChannel&) = delete;

			bool Initialize(const AJADeviceOptions& InDevice, const AJAInputOutputChannelOptions& Options);
			void Uninitialize(TPromise<void> CompletionPromise);

			bool SetAncillaryFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* AncillaryBuffer, uint32_t AncillaryBufferSize);
			bool SetAudioFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* InAudioBuffer, uint32_t InAudioBufferSize);
			bool SetVideoFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* InVideoBuffer, uint32_t InVideoBufferSize);
			bool DMAWriteAudio(const uint8_t* InAudioBuffer, int32_t InAudioBufferSize);
			bool SetVideoFrameData(const AJAOutputFrameBufferData& InFrameData, FRHITexture* RHITexture);
			bool ShouldCaptureThisFrame(::FTimecode Timeocode, uint32 SourceFrameNumber);
			bool GetOutputDimension(uint32_t& OutWidth, uint32_t& OutHeight) const;
			int32_t GetNumAudioSamplesPerFrame(const AJAOutputFrameBufferData& InFrameData) const;

		private:
			std::shared_ptr<OutputChannelThread> ChannelThread;
			std::shared_ptr<DeviceConnection> Device;
			/** VBI Buffer count where [x] will be done transferring on the wire. */
			uint32 BufferFreeIndex[2] = { 0, 0 };
			/** Debug information used to map a VBI event to a UE frame number. */
			uint32 BufferFrameIdMapping[2] = { 0, 0 };
			std::weak_ptr<IOChannelInitialize_DeviceCommand> InitializeCommand;
			uint32_t CurrentOutFrame = 0;
		};

		/* OutputChannel OutputChannelThread
		*****************************************************************************/
		class OutputChannelThread : public ChannelThreadBase
		{
			struct Frame;
			using Super = ChannelThreadBase;
		public:
			OutputChannelThread(const AJADeviceOptions& InDevice, const AJAInputOutputChannelOptions& InOptions);
			~OutputChannelThread();
			
			virtual bool CanInitialize() const override;
			virtual void Uninitialize() override;
			virtual void DeviceThread_Destroy(DeviceConnection::CommandList& InCommandList) override;

			bool SetAncillaryFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* AncillaryBuffer, uint32_t AncillaryBufferSize);
			bool SetAudioFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* AudioBuffer, uint32_t AudioBufferSize);
			bool SetVideoFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* VideoBuffer, uint32_t VideoBufferSize);
			bool SetVideoFrameData(const AJAOutputFrameBufferData& InFrameData, FRHITexture* RHITexture);
			bool ShouldCaptureThisFrame(::FTimecode Timecode, uint32 SourceFrameNumber);
			bool DMAWriteAudio(const uint8_t* InAudioBuffer, int32_t BufferSize);

		protected:
			virtual bool DeviceThread_ConfigureAnc(DeviceConnection::CommandList& InCommandList) override;
			virtual bool DeviceThread_ConfigureAudio(DeviceConnection::CommandList& InCommandList) override;
			virtual bool DeviceThread_ConfigureVideo(DeviceConnection::CommandList& InCommandList) override;

			virtual bool DeviceThread_ConfigureAutoCirculate(DeviceConnection::CommandList& InCommandList) override;
			virtual bool DeviceThread_ConfigurePingPong(DeviceConnection::CommandList& InCommandList) override;

			virtual void Thread_AutoCirculateLoop() override;
			virtual void Thread_PingPongLoop() override;

		private:
			bool IsFrameReadyToBeRead(Frame* InFrame);
			void PushWhenFrameReady(Frame* InFrame);
			Frame* FetchAvailableWritingFrame(const AJAOutputFrameBufferData& InFrameData);
			Frame* Thread_FetchAvailableReadingFrame();
			void Thread_TestInterlacedOutput(Frame* InFrame);

			void Thread_PushAvailableReadingFrame(Frame* InCurrentReadingFrame);
			void ThreadLock_PingPongOutputLoop_Memset0(Frame* InCurrentReadingFrame);

			/** Get the number of bytes separating the audio write and play head. */
			bool Thread_GetAudioOffset(int32& OutAudioOffset);
			bool Thread_GetAudioOffsetInSeconds(double& OutAudioOffset);
			bool Thread_GetAudioOffsetInSamples(int32& OutAudioOffset);
			bool Thread_TransferAudioBuffer(const uint8* InBuffer, int32 InBufferSize);
			bool Thread_HandleAudio(const FString& OutputMethod, Frame* AvailableReadingFrame);
			bool Thread_HandleLostFrameAudio();
			bool Thread_ShouldPauseAudioOutput();

			/** Callback on the vertical interrupt used to keep track of which buffers are currently outputting and to trace debug information. */
			void OnVerticalInterrupt(uint32 SyncCount) const;

			struct Frame
			{
				Frame();
				~Frame();
				Frame(const Frame&) = delete;
				Frame& operator=(const Frame&) = delete;

				uint8_t* AncBuffer;
				uint8_t* AncF2Buffer;
				uint8_t* AudioBuffer;
				uint8_t* VideoBuffer;
				uint32_t CopiedAncBufferSize;
				uint32_t CopiedAncF2BufferSize;
				uint32_t CopiedAudioBufferSize;
				uint32_t CopiedVideoBufferSize;

				uint32_t FrameIdentifier;
				uint32_t FrameIdentifierF2; // used in interlaced
				FTimecode Timecode;
				FTimecode Timecode2;

				bool bAncLineFilled;
				bool bAncF2LineFilled; // used in interlaced
				bool bAudioLineFilled;
				bool bVideoLineFilled;
				bool bVideoF2LineFilled; // used in interlaced

				void Clear();
			};

			uint32_t PingPongDropCount;
			uint32_t LostFrameCounter;

			AJALock FrameLock; // to lock the transfer of frame ptr
			std::vector<Frame*> AllFrames;
			std::vector<Frame*> FrameReadyToRead; // Ready to be copied on the AJA
			std::vector<Frame*> FrameReadyToWrite; // Ready to be used by Unreal Engine


			bool bIsSSE2Available;
			bool bIsFieldAEven;
			bool bIsFirstFetch;

			/** Enabled through Aja.PingPongVersion 1, will wait on a frame received event before falling back on a WaitForVerticalInterrupt call. */
			bool bOutputWithReduceWaiting = false;
			/** Indicates if OutputChannel should trace debug information. (Disabled by default to avoid cluttering Insights Traces). */
			bool bTraceDebugMarkers = false;

			bool bInterlacedTest_ExpectFirstLineToBeWhite;
			uint32_t InterlacedTest_FrameCounter;

			bool bDMABuffersRegistered = false;

			std::atomic<ULWord> CurrentAudioWriteOffset = 0;
			int32_t NumSamplesPerFrame = 0;
			std::atomic<ULWord> AudioPlayheadLastPosition = 0;

			/** Used to protect access to the device when accessed by the audio thread. */
			FCriticalSection DeviceCriticalSection;

			UE::GPUTextureTransfer::TextureTransferPtr TextureTransfer = nullptr;

			/** VBI count where BufferFreeIndex[x] will be done transferring on the wire. */
			uint32 BufferFreeIndex[2] = { 0, 0 };

			/** Debug information used to map a VBI event to a UE frame number. */
			uint32 BufferFrameIdMapping[2] = { 0, 0 };

			/** Event triggered whenever a frame is pushed by Unreal to the FrameReadyToRead queue. */
			FEvent* FrameAvailableEvent = nullptr;

			/** Output framerate. */
			FFrameRate FrameRate;

			/** Handle to the delegate that's triggered when a card emits a vertical interrupt event. */
			FDelegateHandle InterruptEventHandle;
		};
	}
}
