// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSE2MemCpy.h"

#ifdef SSE2_MEM_CPY
#error SSE2_MEM_CPY already defined
#endif

#if defined(_WIN64) && !defined(_M_ARM64)
#define SSE2_MEM_CPY
#endif

#ifdef SSE2_MEM_CPY
#include <intrin.h>
#endif

namespace AJA
{
	namespace Private
	{
		bool IsSSE2Available()
		{
#ifdef SSE2_MEM_CPY
			int CPUInfo[4];
			__cpuid(CPUInfo, 1);
			return (CPUInfo[3] & (1 << 26)) != 0;
#else
			return false;
#endif
		}

		bool IsCorrectlyAlignedForSSE2MemCpy(const void* InDst, const void* InSrc, unsigned int InSize)
		{
#ifdef SSE2_MEM_CPY
			return (((unsigned long long)(InDst) & 15) == 0) && (((unsigned long long)(InSrc) & 15) == 0) && ((InSize & 127) == 0);
#else
			return false;
#endif
		}

		void SSE2MemCpy(const void* InDst, const void* InSrc, unsigned int InSize)
		{
#ifdef SSE2_MEM_CPY
			__m128i* Dst = (__m128i*)InDst;
			__m128i* Src = (__m128i*)InSrc;

			InSize = InSize >> 7;
			for (unsigned int i = 0; i < InSize; i++)
			{
				_mm_prefetch((char *)(Src + 8), 0);
				_mm_prefetch((char *)(Src + 10), 0);
				_mm_prefetch((char *)(Src + 12), 0);
				_mm_prefetch((char *)(Src + 14), 0);
				__m128i m0 = _mm_load_si128(Src + 0);
				__m128i m1 = _mm_load_si128(Src + 1);
				__m128i m2 = _mm_load_si128(Src + 2);
				__m128i m3 = _mm_load_si128(Src + 3);
				__m128i m4 = _mm_load_si128(Src + 4);
				__m128i m5 = _mm_load_si128(Src + 5);
				__m128i m6 = _mm_load_si128(Src + 6);
				__m128i m7 = _mm_load_si128(Src + 7);
				_mm_stream_si128(Dst + 0, m0);
				_mm_stream_si128(Dst + 1, m1);
				_mm_stream_si128(Dst + 2, m2);
				_mm_stream_si128(Dst + 3, m3);
				_mm_stream_si128(Dst + 4, m4);
				_mm_stream_si128(Dst + 5, m5);
				_mm_stream_si128(Dst + 6, m6);
				_mm_stream_si128(Dst + 7, m7);
				Src += 8;
				Dst += 8;
			}
#endif
		}
	}
}

#ifdef SSE2_MEM_CPY
#undef SSE2_MEM_CPY
#endif
