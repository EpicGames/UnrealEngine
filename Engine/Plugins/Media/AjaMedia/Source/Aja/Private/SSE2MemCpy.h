// Copyright Epic Games, Inc. All Rights Reserved.

namespace AJA
{
	namespace Private
	{
		bool IsSSE2Available();
		bool IsCorrectlyAlignedForSSE2MemCpy(const void* InDst, const void* InSrc, unsigned int InSize);
		void SSE2MemCpy(const void* InDst, const void* InSrc, unsigned int InSize);
	}
}
