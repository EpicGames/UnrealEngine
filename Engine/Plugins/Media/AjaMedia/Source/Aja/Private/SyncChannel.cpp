// Copyright Epic Games, Inc. All Rights Reserved.

#include "SyncChannel.h"

#include "Helpers.h"

#include <optional>

namespace AJA
{
	namespace Private
	{
		/* SyncChannel implementation
		*****************************************************************************/
		bool SyncChannel::Initialize(const AJADeviceOptions& InDeviceOption, const AJASyncChannelOptions& InSyncOptions)
		{
			Uninitialize();
			SyncImplementation.reset(new SyncChannelImpl(InDeviceOption, InSyncOptions));

			bool bResult = SyncImplementation->CanInitialize();
			if (bResult)
			{
				Device = DeviceCache::GetDevice(InDeviceOption);
				std::shared_ptr<SyncChannelInitialize_DeviceCommand> SharedCommand(new SyncChannelInitialize_DeviceCommand(Device, SyncImplementation));
				InitializeCommand = SharedCommand;
				Device->AddCommand(std::static_pointer_cast<DeviceCommand>(SharedCommand));
			}

			return bResult;
		}

		void SyncChannel::Uninitialize()
		{
			if (SyncImplementation)
			{
				std::shared_ptr<SyncChannelInitialize_DeviceCommand> SharedCommand = InitializeCommand.lock();
				if (SharedCommand && Device)
				{
					Device->RemoveCommand(std::static_pointer_cast<DeviceCommand>(SharedCommand));
				}

				SyncImplementation->Uninitialize(); // delete is done in SyncChannelUninitialize_DeviceCommand completed

				if (Device)
				{
					std::shared_ptr<SyncChannelUninitialize_DeviceCommand> UninitializeCommand;
					UninitializeCommand.reset(new SyncChannelUninitialize_DeviceCommand(Device, SyncImplementation));
					Device->AddCommand(std::static_pointer_cast<DeviceCommand>(UninitializeCommand));
				}
			}

			InitializeCommand.reset();
			SyncImplementation.reset();
			Device.reset();
		}

		bool SyncChannel::WaitForSync()
		{
			if (SyncImplementation)
			{
				return SyncImplementation->WaitForSync();
			}
			return false;
		}

		bool SyncChannel::GetTimecode(FTimecode& OutTimecode)
		{
			if (SyncImplementation)
			{
				return SyncImplementation->GetTimecode(OutTimecode);
			}
			return false;
		}

		bool SyncChannel::GetSyncCount(uint32_t& OutCount) const
		{
			if (SyncImplementation)
			{
				return SyncImplementation->GetSyncCount(OutCount);
			}
			return false;
		}

		bool SyncChannel::GetVideoFormat(NTV2VideoFormat& OutVideoFormat)
		{
			if (SyncImplementation)
			{
				return SyncImplementation->GetVideoFormat(OutVideoFormat);
			}
			return false;
		}

		/* SyncChannelInitialize_DeviceCommand implementation
		*****************************************************************************/
		SyncChannelInitialize_DeviceCommand::SyncChannelInitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<SyncChannelImpl>& InChannel)
			: DeviceCommand(InConnection)
			, SyncImplementation(InChannel)
		{}

		void SyncChannelInitialize_DeviceCommand::Execute(DeviceConnection::CommandList& InCommandList)
		{
			std::shared_ptr<SyncChannelImpl> Shared = SyncImplementation.lock();
			if (Shared) // could have been Uninitialize before we have time to execute it
			{
				if (!Shared->Thread_Initialize(InCommandList))
				{
					Shared->Thread_Destroy(InCommandList);
				}
			}
			SyncImplementation.reset();
		}

		/* SyncChannelUninitialize_DeviceCommand implementation
		*****************************************************************************/
		SyncChannelUninitialize_DeviceCommand::SyncChannelUninitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<SyncChannelImpl>& InChannel)
			: DeviceCommand(InConnection)
			, SyncImplementation(InChannel)
		{}

		void SyncChannelUninitialize_DeviceCommand::Execute(DeviceConnection::CommandList& InCommandList)
		{
			// keep a shared pointer to prevent the destruction of the SyncChannel until its turn comes up
			SyncImplementation->Thread_Destroy(InCommandList);
			SyncImplementation.reset();
		}

		/* SyncChannelImpl implementation
		*****************************************************************************/
		SyncChannelImpl::SyncChannelImpl(const AJADeviceOptions& InDevice, const AJASyncChannelOptions& InOptions)
			: DeviceOption(InDevice)
			, SyncOption(InOptions)
			//, Card(nullptr)
			, Channel(NTV2Channel::NTV2_CHANNEL_INVALID)
			, DesiredVideoFormat(NTV2VideoFormat::NTV2_FORMAT_UNKNOWN)
			, BaseFrameIndex(0)
			, bRegisterChannel(false)
			, bRegisterReference(false)
			, bHaveTimecodeIssue(false)
			, PreviousVerticalInterruptCount(0)
			, bStopRequested(false)
		{
			if (InOptions.ChannelIndex > NTV2_MAX_NUM_CHANNELS || InOptions.ChannelIndex < 1)
			{
				UE_LOGF(LogAjaCore, Error, "SyncChannel: The port index '%d' is invalid.\n", InOptions.ChannelIndex);
			}
			else
			{
				Channel = (NTV2Channel)(InOptions.ChannelIndex - 1);
				InputSource = GetNTV2InputSourceForIndex(Channel, Helpers::IsHdmiTransport(InOptions.TransportType) ? NTV2_INPUTSOURCES_HDMI : NTV2_INPUTSOURCES_SDI);
			}
		}

		SyncChannelImpl::~SyncChannelImpl()
		{
			//assert(Card == nullptr);
		}

		bool SyncChannelImpl::CanInitialize() const
		{
			return Channel != NTV2_CHANNEL_INVALID;
		}

		bool SyncChannelImpl::Thread_Initialize(DeviceConnection::CommandList& InCommandList)
		{
			bool bResult = Thread_Configure(InCommandList);
			Thread_OnInitializationCompleted(bResult);
			return bResult;
		}

		void SyncChannelImpl::Uninitialize()
		{
			AJAAutoLock AutoLock(&Lock);
			SyncOption.CallbackInterface = nullptr;
			bStopRequested = true;
		}

		void SyncChannelImpl::Thread_Destroy(DeviceConnection::CommandList& InCommandList)
		{
			if (Device) // Device is valid when we went in the initialize
			{
				if (bRegisterReference)
				{
					InCommandList.UnregisterReference(Channel);
				}

				if (bRegisterChannel)
				{
					bool bAsOwner = false;
					InCommandList.UnregisterChannel(Channel, bAsOwner);
				}
			}

			//delete Card;

			Device.reset();
			//Card = nullptr;
			bRegisterReference = false;
			bRegisterChannel = false;
		}

		void SyncChannelImpl::Thread_OnInitializationCompleted(bool bSucceed)
		{
			AJAAutoLock AutoLock(&Lock);
			if (SyncOption.CallbackInterface && !bStopRequested)
			{
				SyncOption.CallbackInterface->OnInitializationCompleted(bSucceed);
			}
		}

		bool SyncChannelImpl::Thread_Configure(DeviceConnection::CommandList& InCommandList)
		{
			if (bStopRequested)
			{
				return false;
			}
			if (!CanInitialize())
			{
				return false;
			}
			
			assert(Device == nullptr);
			Device = DeviceCache::GetDevice(DeviceOption);
			if (Device == nullptr)
			{
				return false;
			}

			if (bStopRequested)
			{
				return false;
			}

			//assert(Card == nullptr);
			//Card = DeviceConnection::NewCNTV2Card(DeviceOption.DeviceIndex);
			//if (Card == nullptr)
			//{
			//	return false;
			//}

			if (bStopRequested)
			{
				return false;
			}

			if (!Helpers::TryVideoFormatIndexToNTV2VideoFormat(SyncOption.VideoFormatIndex, DesiredVideoFormat))
			{
				UE_LOGF(LogAjaCore, Error, "SyncChannel: The expected video format is invalid for %d.\n", uint32_t(Channel) + 1);
				return false;
			}

			if (!Helpers::ConvertTransportForDevice(Device->GetCard(), DeviceOption.DeviceIndex, SyncOption.TransportType, DesiredVideoFormat))
			{
				return false;
			}

			const bool bRegisteredAsInput = !SyncOption.bOutput;
			constexpr bool bConnectChannel = false;
			constexpr bool bAsOwner = false;
			constexpr bool bAsGenlock = true;
			const EPixelFormat DefaultPixelFormat = EPixelFormat::PF_8BIT_YCBCR;
			if (!InCommandList.RegisterChannel(SyncOption.TransportType, InputSource, Channel, bRegisteredAsInput, bAsGenlock, bConnectChannel, SyncOption.TimecodeFormat, DefaultPixelFormat, DesiredVideoFormat, bAsOwner, SyncOption.bAutoDetectFormat))
			{
				return false;
			}
			bRegisterChannel = true;

			if (SyncOption.bOutput)
			{
				if (!InCommandList.RegisterReference(EAJAReferenceType::EAJA_REFERENCETYPE_EXTERNAL, Channel))
				{
					return false;
				}
				bRegisterReference = true;
			}

			if (bStopRequested)
			{
				return false;
			}

			if (!Device->WaitForInputOrOutputInterrupt(Channel, 12 * 2))
			{
				UE_LOGF(LogAjaCore, Error, "SyncChannel: Was not able to lock on device %s for channel %d.\n", GetDevice().GetDisplayName().c_str(), uint32_t(Channel) + 1);
				return false;
			}

			if (bStopRequested)
			{
				return false;
			}

			return true;
		}

		bool SyncChannelImpl::WaitForSync()
		{
			if (!SyncOption.bOutput)
			{
				std::string FailureReason;
				const NTV2InputSource Input = Helpers::IsHdmiTransport(SyncOption.TransportType) ? GetNTV2HDMIInputSourceForIndex(Channel) : GetNTV2InputSourceForIndex(Channel);
				std::optional<NTV2VideoFormat> FoundFormat = Helpers::GetInputVideoFormat(Device->GetCard(), SyncOption.TransportType, Channel, Input, DesiredVideoFormat, false, FailureReason);

				if (!FoundFormat.has_value())
				{
					UE_LOGF(LogAjaCore, Error, "Sync: The VideoFormat was invalid for channel %d on device %s. %s\n"
						, uint32_t(Channel) + 1
						, GetDevice().GetDisplayName().c_str()
						, FailureReason.c_str()
					);
					return false;
				}

				if (!Helpers::CompareFormats(DesiredVideoFormat, FoundFormat.value(), FailureReason))
				{
					if (SyncOption.bAutoDetectFormat)
					{
						DesiredVideoFormat = FoundFormat.value();
						Device->SetFormat(Channel, FoundFormat.value());
					}
					else
					{
						UE_LOGF(LogAjaCore, Error, "Sync: The VideoFormat changed for channel %d on device %s. %s\n"
							, uint32_t(Channel) + 1
							, GetDevice().GetDisplayName().c_str()
							, FailureReason.c_str()
						);
						return false;
					}
				}
			}
				
			bool bResult = true;
			if (SyncOption.bWaitForFrameToBeReady)
			{
				bResult = Device->WaitForInputFrameReceived(Channel);
			}
			else
			{
				// Wait for 1 field (interlace == 2, progressive == 1). In PSF, we want to wait for 2 fields.
				const DeviceConnection::EInterruptType InterruptType = ::IsPSF(DesiredVideoFormat) ? DeviceConnection::EInterruptType::InputField : DeviceConnection::EInterruptType::VerticalInterrupt;
				bResult = Device->WaitForInputOrOutputInterrupt(Channel, 1, InterruptType);
			}

			if (!bResult && !SyncOption.bStopOnCardTimeout)
			{
				// If bStopOnCardTimeout is disabled, then we want sync to keep going if the SDK timeouts while waiting for a card interupt.
				UE_LOGF(LogAjaCore, Error, "SyncChannel: Was not able to wait for an interrupt signal on device %s for channel %d.\n", GetDevice().GetDisplayName().c_str(), uint32_t(Channel) + 1);
				bResult = true;
			}

			return bResult;
		}

		bool SyncChannelImpl::GetTimecode(FTimecode& OutTimecode)
		{
			if (!SyncOption.bOutput)
			{
				uint32_t ReadBackIn;
				if (SyncOption.TimecodeFormat != ETimecodeFormat::TCF_None && !bHaveTimecodeIssue)
				{
					AJA_CHECK(GetDevice().GetInputFrame(Channel, ReadBackIn));
					ReadBackIn ^= 1; // Grab the next frame (processing frame)

					FTimecode NewTimecode;
					constexpr bool bLogError = true;
					bHaveTimecodeIssue = !Helpers::GetTimecode(Device->GetCard(), Channel, DesiredVideoFormat, ReadBackIn, SyncOption.TimecodeFormat, bLogError, NewTimecode);
					
					OutTimecode = Helpers::AdjustTimecodeForUE(Device->GetCard(), Channel, DesiredVideoFormat, NewTimecode, PreviousTimecode, PreviousVerticalInterruptCount);
					PreviousTimecode = NewTimecode;

					return !bHaveTimecodeIssue;
				}
			}

			return false;
		}

		bool SyncChannelImpl::GetSyncCount(uint32_t& OutCount) const
		{
			if (!SyncOption.bOutput)
			{
				return Device->GetCard()->GetInputVerticalInterruptCount(OutCount, Channel);
			}
			else
			{
				return Device->GetCard()->GetOutputVerticalInterruptCount(OutCount, Channel);
			}
		}

		bool SyncChannelImpl::GetVideoFormat(NTV2VideoFormat& OutVideoFormat)
		{
			// Reference in.
			if (SyncOption.bOutput)
			{
				OutVideoFormat = Device->GetCard()->GetReferenceVideoFormat();
				return OutVideoFormat != NTV2_FORMAT_UNKNOWN;
			}

			// SDI or HDMI

			std::string FailureReason;
			NTV2InputSource LocalInputSource;
				
			if (Helpers::IsHdmiTransport(SyncOption.TransportType))
			{
				LocalInputSource = GetNTV2HDMIInputSourceForIndex(Channel);
			}
			else
			{
				LocalInputSource = GetNTV2InputSourceForIndex(Channel);
			}

			constexpr bool bSetSDIConversion = false;

			return Helpers::GetInputVideoFormat(
				Device->GetCard(), 
				SyncOption.TransportType, 
				Channel, 
				LocalInputSource,
				DesiredVideoFormat, 
				OutVideoFormat, 
				bSetSDIConversion,
				FailureReason);
		}
	}
}
