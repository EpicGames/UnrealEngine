// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Device.h"
#include "Helpers.h"

#include <memory>

namespace AJA
{
	namespace Private
	{
		class SyncChannel;
		class SyncChannelImpl;
		struct SyncChannelInitialize_DeviceCommand;
		struct SyncChannelUninitialize_DeviceCommand;

		/* SyncChannel definition
		*****************************************************************************/
		class SyncChannel
		{
		public:
			SyncChannel() = default;
			SyncChannel(const SyncChannel&) = delete;
			SyncChannel& operator=(const SyncChannel&) = delete;

			bool Initialize(const AJADeviceOptions& InDevice, const AJASyncChannelOptions& InOptions);
			void Uninitialize();

			bool WaitForSync();
			bool GetTimecode(FTimecode& OutTimecode);
			bool GetSyncCount(uint32_t& OutCount) const;
			bool GetVideoFormat(NTV2VideoFormat& OutVideoFormat);

		private:
			std::shared_ptr<SyncChannelImpl> SyncImplementation;
			std::shared_ptr<DeviceConnection> Device;
			std::weak_ptr<SyncChannelInitialize_DeviceCommand> InitializeCommand;
		};

		/* SyncChannelInitialize_DeviceCommand definition
		*****************************************************************************/
		struct SyncChannelInitialize_DeviceCommand : DeviceCommand
		{
			SyncChannelInitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<SyncChannelImpl>& InChannel);

			virtual void Execute(DeviceConnection::CommandList& InCommandList) override;

			std::weak_ptr<SyncChannelImpl> SyncImplementation;
		};

		/* SyncChannelUninitialize_DeviceCommand definition
		*****************************************************************************/
		struct SyncChannelUninitialize_DeviceCommand : DeviceCommand
		{
			SyncChannelUninitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<SyncChannelImpl>& InChannel);

			virtual void Execute(DeviceConnection::CommandList& InCommandList) override;

			std::shared_ptr<SyncChannelImpl> SyncImplementation;
		};


		/* SyncChannelImpl definition
		*****************************************************************************/
		class SyncChannelImpl
		{
		public:
			SyncChannelImpl() = delete;
			SyncChannelImpl(const AJADeviceOptions& InDevice, const AJASyncChannelOptions& InOptions);
			SyncChannelImpl(const SyncChannelImpl&) = delete;
			SyncChannelImpl& operator=(const SyncChannelImpl&) = delete;
			virtual ~SyncChannelImpl();

			bool CanInitialize() const;
			bool Thread_Initialize(DeviceConnection::CommandList& InCommandList);
			void Uninitialize();
			void Thread_Destroy(DeviceConnection::CommandList& InCommandList);

			bool WaitForSync();
			bool GetTimecode(FTimecode& OutTimecode);
			bool GetSyncCount(uint32_t& OutCount) const;
			bool GetVideoFormat(NTV2VideoFormat& OutVideoFormat);

		private:
			CNTV2Card& GetDevice() { return *Device->GetCard(); }
			//CNTV2Card& GetDevice() { assert(Card); return *Card; }
			//CNTV2Card& GetDevice() const { assert(Card); return *Card; }

		private:
			AJALock Lock; // for callback
			AJADeviceOptions DeviceOption;
			AJASyncChannelOptions SyncOption;

			std::shared_ptr<DeviceConnection> Device;
			//CNTV2Card* Card;

			NTV2Channel Channel;
			NTV2InputSource InputSource;
			NTV2VideoFormat DesiredVideoFormat;

			uint32_t BaseFrameIndex;

			bool bRegisterChannel;
			bool bRegisterReference;
			bool bHaveTimecodeIssue;
			FTimecode PreviousTimecode;
			ULWord PreviousVerticalInterruptCount;

		private:

			bool Thread_Configure(DeviceConnection::CommandList& InCommandList);
			void Thread_OnInitializationCompleted(bool bSucceed);

		private:
			volatile bool bStopRequested;
		};
	}
}
