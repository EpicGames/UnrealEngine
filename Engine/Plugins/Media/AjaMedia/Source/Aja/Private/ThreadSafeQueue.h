// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Helpers.h"

#include <algorithm>
#include <vector>

namespace AJA
{
	namespace Private
	{
		template<class T>
		class ThreadSafeQueue
		{
		public:
			void Push(const T& InValue)
			{
				AJAAutoLock AutoLock(&Lock);
				List.push_back(InValue);
			}

			bool Pop(T& OutValue)
			{
				AJAAutoLock AutoLock(&Lock);
				if (!List.empty())
				{
					OutValue = List.front();
					List.erase(List.begin());
					return true;
				}
				return false;
			}

			void Erase(const T& InValue)
			{
				AJAAutoLock AutoLock(&Lock);
				auto Found = std::find(List.begin(), List.end(), InValue);
				if (Found != List.end())
				{
					List.erase(Found);
				}
			}

			size_t Size() const
			{
				return List.size();
			}

		private:
			AJALock Lock;
			std::vector<T> List;
		};
	}
}
