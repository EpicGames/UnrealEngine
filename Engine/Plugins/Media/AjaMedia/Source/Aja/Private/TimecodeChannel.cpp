// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeChannel.h"

#include "Helpers.h"


namespace AJA
{
	namespace Private
	{
		/* TimecodeChannel implementation
		*****************************************************************************/
		bool TimecodeChannel::Initialize(const AJADeviceOptions& InDeviceOption, const AJATimecodeChannelOptions& InTimecodeOptions)
		{
			Uninitialize();
			TimecodeImplementation.reset(new TimecodeChannelImpl(InDeviceOption, InTimecodeOptions));

			bool bResult = TimecodeImplementation->CanInitialize();
			if (bResult)
			{
				Device = DeviceCache::GetDevice(InDeviceOption);
				std::shared_ptr<TimecodeChannelInitialize_DeviceCommand> SharedCommand(new TimecodeChannelInitialize_DeviceCommand(Device, TimecodeImplementation));
				InitializeCommand = SharedCommand;
				Device->AddCommand(std::static_pointer_cast<DeviceCommand>(SharedCommand));
			}

			return bResult;
		}

		void TimecodeChannel::Uninitialize()
		{
			if (TimecodeImplementation)
			{
				std::shared_ptr<TimecodeChannelInitialize_DeviceCommand> SharedCommand = InitializeCommand.lock();
				if (SharedCommand && Device)
				{
					Device->RemoveCommand(std::static_pointer_cast<DeviceCommand>(SharedCommand));
				}

				TimecodeImplementation->Uninitialize(); // delete is done in TimecodeChannelUninitialize_DeviceCommand completed

				if (Device)
				{
					std::shared_ptr<TimecodeChannelUninitialize_DeviceCommand> UninitializeCommand;
					UninitializeCommand.reset(new TimecodeChannelUninitialize_DeviceCommand(Device, TimecodeImplementation));
					Device->AddCommand(std::static_pointer_cast<DeviceCommand>(UninitializeCommand));
				}
			}

			InitializeCommand.reset();
			TimecodeImplementation.reset();
			Device.reset();
		}

		bool TimecodeChannel::GetTimecode(FTimecode& OutTimecode)
		{
			if (TimecodeImplementation)
			{
				return TimecodeImplementation->GetTimecode(OutTimecode);
			}
			return false;
		}

		/* TimecodeChannelInitialize_DeviceCommand implementation
		*****************************************************************************/
		TimecodeChannelInitialize_DeviceCommand::TimecodeChannelInitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<TimecodeChannelImpl>& InChannel)
			: DeviceCommand(InConnection)
			, TimecodeImplementation(InChannel)
		{}

		void TimecodeChannelInitialize_DeviceCommand::Execute(DeviceConnection::CommandList& InCommandList)
		{
			std::shared_ptr<TimecodeChannelImpl> Shared = TimecodeImplementation.lock();
			if (Shared) // could have been Uninitialize before we have time to execute it
			{
				if (!Shared->Thread_Initialize(InCommandList))
				{
					Shared->Thread_Destroy(InCommandList);
				}
			}
			TimecodeImplementation.reset();
		}

		/* TimecodeChannelUninitialize_DeviceCommand implementation
		*****************************************************************************/
		TimecodeChannelUninitialize_DeviceCommand::TimecodeChannelUninitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<TimecodeChannelImpl>& InChannel)
			: DeviceCommand(InConnection)
			, TimecodeImplementation(InChannel)
		{}

		void TimecodeChannelUninitialize_DeviceCommand::Execute(DeviceConnection::CommandList& InCommandList)
		{
			// keep a shared pointer to prevent the destruction of the TimecodeChannel until its turn comes up
			TimecodeImplementation->Thread_Destroy(InCommandList);
			TimecodeImplementation.reset();
		}

		/* TimecodeChannelImpl implementation
		*****************************************************************************/
		TimecodeChannelImpl::TimecodeChannelImpl(const AJADeviceOptions& InDevice, const AJATimecodeChannelOptions& InOptions)
			: DeviceOption(InDevice)
			, TimecodeOption(InOptions)
			//, Card(nullptr)
			, Channel(NTV2Channel::NTV2_CHANNEL_INVALID)
			, DesiredVideoFormat(NTV2VideoFormat::NTV2_FORMAT_UNKNOWN)
			, bRegisteredDedicatedLTC(false)
			, bRegisteredReferenceLTC(false)
			, bRegisteredChannel(false)
			, bHaveTimecodeIssue(false)
			, PreviousVerticalInterruptCount(0)
			, bStopRequested(false)
		{
			if (!TimecodeOption.bUseDedicatedPin)
			{
				if (InOptions.ChannelIndex > NTV2_MAX_NUM_CHANNELS || InOptions.ChannelIndex < 1)
				{
					UE_LOGF(LogAjaCore, Error, "SyncChannel: The port index '%d' is invalid.\n", InOptions.ChannelIndex);
				}
				else
				{
					Channel = (NTV2Channel)(InOptions.ChannelIndex - 1);
					InputSource = GetNTV2InputSourceForIndex(Channel, Helpers::IsHdmiTransport(InOptions.TransportType) ? NTV2_INPUTSOURCES_HDMI : NTV2_INPUTSOURCES_SDI);
				}
			}
		}

		TimecodeChannelImpl::~TimecodeChannelImpl()
		{
			//assert(Card == nullptr);
		}

		bool TimecodeChannelImpl::CanInitialize() const
		{
			return TimecodeOption.bUseDedicatedPin || (Channel != NTV2_CHANNEL_INVALID);
		}

		bool TimecodeChannelImpl::Thread_Initialize(DeviceConnection::CommandList& InCommandList)
		{
			bool bResult = Thread_Configure(InCommandList);
			Thread_OnInitializationCompleted(bResult);
			return bResult;
		}

		void TimecodeChannelImpl::Uninitialize()
		{
			AJAAutoLock AutoLock(&Lock);
			TimecodeOption.CallbackInterface = nullptr;
			bStopRequested = true;
		}

		void TimecodeChannelImpl::Thread_Destroy(DeviceConnection::CommandList& InCommandList)
		{
			if (Device) // Device is valid when we went in the initialize
			{
				if (bRegisteredChannel)
				{
					const bool bAsOwner = false;
					InCommandList.UnregisterChannel(Channel, bAsOwner);
				}

				if (bRegisteredDedicatedLTC)
				{
					InCommandList.UnregisterAnalogLtc(bRegisteredReferenceLTC);
				}
			}

			//delete Card;

			Device.reset();
			//Card = nullptr;
			bRegisteredReferenceLTC = false;
			bRegisteredChannel = false;
			bRegisteredDedicatedLTC;
		}

		void TimecodeChannelImpl::Thread_OnInitializationCompleted(bool bSucceed)
		{
			AJAAutoLock AutoLock(&Lock);
			if (TimecodeOption.CallbackInterface && !bStopRequested)
			{
				TimecodeOption.CallbackInterface->OnInitializationCompleted(bSucceed);
			}
		}

		bool TimecodeChannelImpl::Thread_Configure(DeviceConnection::CommandList& InCommandList)
		{
			if (bStopRequested)
			{
				return false;
			}
			if (!CanInitialize())
			{
				return false;
			}
			
			assert(Device == nullptr);
			Device = DeviceCache::GetDevice(DeviceOption);
			if (Device == nullptr)
			{
				return false;
			}

			if (bStopRequested)
			{
				return false;
			}

			//assert(Card == nullptr);
			//Card = DeviceConnection::NewCNTV2Card(DeviceOption.DeviceIndex);
			//if (Card == nullptr)
			//{
			//	return false;
			//}

			if (bStopRequested)
			{
				return false;
			}

			if (TimecodeOption.bUseDedicatedPin)
			{
				//Convert LTC index to enum + validate 
				EAnalogLTCSource LtcSource = EAnalogLTCSource::LTC1;
				switch (TimecodeOption.LTCSourceIndex)
				{
				case 1: LtcSource = EAnalogLTCSource::LTC1; break;
				case 2: LtcSource = EAnalogLTCSource::LTC2; break;
				default:
					UE_LOGF(LogAjaCore, Error, "TimecodeChannel: The LTC source Index is invalid on device %s.\n", GetDevice().GetDisplayName().c_str());
					return false;
				}

				if (!InCommandList.RegisterAnalogLtc(LtcSource, Helpers::ConvertToFrameRate(TimecodeOption.LTCFrameRateNumerator, TimecodeOption.LTCFrameRateDenominator), TimecodeOption.bReadTimecodeFromReferenceIn))
				{
					return false;
				}
				bRegisteredDedicatedLTC = true;
				bRegisteredReferenceLTC = TimecodeOption.bReadTimecodeFromReferenceIn;
			}
			else
			{
				bRegisteredDedicatedLTC = false;

				if (!Helpers::TryVideoFormatIndexToNTV2VideoFormat(TimecodeOption.VideoFormatIndex, DesiredVideoFormat))
				{
					UE_LOGF(LogAjaCore, Error, "TimecodeChannel: The expected video format is invalid for %d.\n", uint32_t(Channel) + 1);
					return false;
				}

				if (!Helpers::ConvertTransportForDevice(Device->GetCard(), DeviceOption.DeviceIndex, TimecodeOption.TransportType, DesiredVideoFormat))
				{
					return false;
				}

				constexpr bool bRegisteredAsInput = true;
				constexpr bool bConnectChannel = false;
				constexpr bool bAsOwner = false;
				constexpr bool bAsGenlock = false;

				const EPixelFormat DefaultPixelFormat = EPixelFormat::PF_8BIT_YCBCR;
				if (!InCommandList.RegisterChannel(TimecodeOption.TransportType, InputSource, Channel, bRegisteredAsInput, bAsGenlock, bConnectChannel, TimecodeOption.TimecodeFormat, DefaultPixelFormat, DesiredVideoFormat, bAsOwner, TimecodeOption.bAutoDetectFormat))
				{
					return false;
				}
				bRegisteredChannel = true;

				if (bStopRequested)
				{
					return false;
				}

				if (!Device->WaitForInputOrOutputInterrupt(Channel, 12 * 2))
				{
					UE_LOGF(LogAjaCore, Error, "TimecodeChannel: Was not able to lock on device %s for channel %d.\n", GetDevice().GetDisplayName().c_str(), uint32_t(Channel) + 1);
					return false;
				}
			}

			if (bStopRequested)
			{
				return false;
			}

			return true;
		}

		bool TimecodeChannelImpl::GetTimecode(FTimecode& OutTimecode)
		{
			if (std::chrono::system_clock::now() - LastLogTime > SecondsBetweenLogs)
			{
				bLogTimecodeError = true;
			}

			if (!bHaveTimecodeIssue)
			{
				if (TimecodeOption.bUseDedicatedPin)
				{

					bHaveTimecodeIssue = !Helpers::GetTimecode(Device->GetCard(), TimecodeOption.LTCSourceIndex == 1 ? EAnalogLTCSource::LTC1 : EAnalogLTCSource::LTC2, NTV2VideoFormat::NTV2_FORMAT_UNKNOWN, bLogTimecodeError, OutTimecode);
					return !bHaveTimecodeIssue;
				}
				else if (TimecodeOption.TimecodeFormat != ETimecodeFormat::TCF_None)
				{
					uint32_t ReadBackIn;
					AJA_CHECK(GetDevice().GetInputFrame(Channel, ReadBackIn));
					ReadBackIn ^= 1; // Grab the next frame (processing frame)

					FTimecode NewTimecode;
					bHaveTimecodeIssue = !Helpers::GetTimecode(Device->GetCard(), Channel, DesiredVideoFormat, ReadBackIn, TimecodeOption.TimecodeFormat, bLogTimecodeError, NewTimecode);

					OutTimecode = Helpers::AdjustTimecodeForUE(Device->GetCard(), Channel, DesiredVideoFormat, NewTimecode, PreviousTimecode, PreviousVerticalInterruptCount);
					PreviousTimecode = NewTimecode;

					return !bHaveTimecodeIssue;
				}
			}

			if (bHaveTimecodeIssue)
			{
				LastLogTime = std::chrono::system_clock::now();
			}

			return bHaveTimecodeIssue;
		}
	}
}
