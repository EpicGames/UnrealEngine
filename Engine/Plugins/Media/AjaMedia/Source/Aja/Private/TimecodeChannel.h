// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Device.h"
#include "Helpers.h"

#include <chrono>
#include <ctime>
#include <memory>

namespace AJA
{
	namespace Private
	{
		class TimecodeChannel;
		class TimecodeChannelImpl;
		struct TimecodeChannelInitialize_DeviceCommand;
		struct TimecodeChannelUninitialize_DeviceCommand;

		/* TimecodeChannel definition
		*****************************************************************************/
		class TimecodeChannel
		{
		public:
			TimecodeChannel() = default;
			TimecodeChannel(const TimecodeChannel&) = delete;
			TimecodeChannel& operator=(const TimecodeChannel&) = delete;

			bool Initialize(const AJADeviceOptions& InDevice, const AJATimecodeChannelOptions& InOptions);
			void Uninitialize();

			bool GetTimecode(FTimecode& OutTimecode);

		private:
			std::shared_ptr<TimecodeChannelImpl> TimecodeImplementation;
			std::shared_ptr<DeviceConnection> Device;
			std::weak_ptr<TimecodeChannelInitialize_DeviceCommand> InitializeCommand;
		};

		/* TimecodeChannelInitialize_DeviceCommand definition
		*****************************************************************************/
		struct TimecodeChannelInitialize_DeviceCommand : DeviceCommand
		{
			TimecodeChannelInitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<TimecodeChannelImpl>& InChannel);

			virtual void Execute(DeviceConnection::CommandList& InCommandList) override;

			std::weak_ptr<TimecodeChannelImpl> TimecodeImplementation;
		};

		/* TimecodeChannelUninitialize_DeviceCommand definition
		*****************************************************************************/
		struct TimecodeChannelUninitialize_DeviceCommand : DeviceCommand
		{
			TimecodeChannelUninitialize_DeviceCommand(const std::shared_ptr<DeviceConnection>& InConnection, const std::weak_ptr<TimecodeChannelImpl>& InChannel);

			virtual void Execute(DeviceConnection::CommandList& InCommandList) override;

			std::shared_ptr<TimecodeChannelImpl> TimecodeImplementation;
		};


		/* TimecodeChannelImpl definition
		*****************************************************************************/
		class TimecodeChannelImpl
		{
		public:
			TimecodeChannelImpl() = delete;
			TimecodeChannelImpl(const AJADeviceOptions& InDevice, const AJATimecodeChannelOptions& InOptions);
			TimecodeChannelImpl(const TimecodeChannelImpl&) = delete;
			TimecodeChannelImpl& operator=(const TimecodeChannelImpl&) = delete;
			virtual ~TimecodeChannelImpl();

			bool CanInitialize() const;
			bool Thread_Initialize(DeviceConnection::CommandList& InCommandList);
			void Uninitialize();
			void Thread_Destroy(DeviceConnection::CommandList& InCommandList);

			bool GetTimecode(FTimecode& OutTimecode);

		private:
			CNTV2Card& GetDevice() { return *Device->GetCard(); }
			//CNTV2Card& GetDevice() { assert(Card); return *Card; }
			//CNTV2Card& GetDevice() const { assert(Card); return *Card; }

		private:
			AJALock Lock; // for callback
			AJADeviceOptions DeviceOption;
			AJATimecodeChannelOptions TimecodeOption;

			std::shared_ptr<DeviceConnection> Device;
			//CNTV2Card* Card;

			NTV2Channel Channel;
			NTV2InputSource InputSource;
			NTV2VideoFormat DesiredVideoFormat;

			bool bRegisteredDedicatedLTC;
			bool bRegisteredReferenceLTC;
			bool bRegisteredChannel;
			bool bHaveTimecodeIssue;
			FTimecode PreviousTimecode;
			ULWord PreviousVerticalInterruptCount;

		private:

			bool Thread_Configure(DeviceConnection::CommandList& InCommandList);
			void Thread_OnInitializationCompleted(bool bSucceed);

		private:
			volatile bool bStopRequested;
			std::chrono::system_clock::time_point LastLogTime;
			std::chrono::duration<double> SecondsBetweenLogs = std::chrono::duration<double>(5.0);
			bool bLogTimecodeError = true;
			
		};
	}
}
