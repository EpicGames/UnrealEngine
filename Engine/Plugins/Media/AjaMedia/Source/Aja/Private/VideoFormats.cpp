// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoFormats.h"

#include <algorithm>

namespace AJA
{
	namespace Private
	{
		/* VideoFormatsScanner implementation
		*****************************************************************************/
		VideoFormatsScanner::VideoFormatsScanner(int32_t InDeviceId)
		{
			std::unique_ptr<CNTV2DeviceScanner> Scanner = std::make_unique<CNTV2DeviceScanner>();
			if (InDeviceId < Scanner->GetNumDevices())
			{
				NTV2DeviceInfo DeviceInfo;
				Scanner->GetDeviceInfo(InDeviceId, DeviceInfo, true);

				NTV2VideoFormatSet SetFormats;
				::NTV2DeviceGetSupportedVideoFormats(DeviceInfo.deviceID, SetFormats);

				for (const NTV2VideoFormat Format : SetFormats)
				{
					FormatList.emplace_back(GetVideoFormat(Format));
				}
			}
		}

		AJAVideoFormats::VideoFormatDescriptor VideoFormatsScanner::GetVideoFormat(FAJAVideoFormat InAjaVideoFormat)
		{
			NTV2VideoFormat AjaVideoFormat = static_cast<NTV2VideoFormat>(InAjaVideoFormat);
			if (!NTV2_IS_VALID_VIDEO_FORMAT(AjaVideoFormat))
			{
				return AJAVideoFormats::VideoFormatDescriptor();
			}

			AJAVideoFormats::VideoFormatDescriptor NewFormat;
			NewFormat.VideoFormatIndex = AjaVideoFormat;
			::GetFramesPerSecond(::GetNTV2FrameRateFromVideoFormat(AjaVideoFormat), NewFormat.FrameRateNumerator, NewFormat.FrameRateDenominator);
			NewFormat.ResolutionWidth = ::GetDisplayWidth(AjaVideoFormat);
			NewFormat.ResolutionHeight = ::GetDisplayHeight(AjaVideoFormat);
			NewFormat.bIsProgressiveStandard = ::IsProgressiveTransport(AjaVideoFormat); //use NTV2_IS_PROGRESSIVE_STANDARD
			NewFormat.bIsInterlacedStandard = !::IsProgressivePicture(AjaVideoFormat); //use NTV2_VIDEO_FORMAT_HAS_PROGRESSIVE_PICTURE
			NewFormat.bIsPsfStandard = ::IsPSF(AjaVideoFormat);
			NewFormat.bIsVideoFormatA = ::IsVideoFormatA(AjaVideoFormat);
			NewFormat.bIsVideoFormatB = ::IsVideoFormatB(AjaVideoFormat);
			NewFormat.bIs372DualLink = NTV2_IS_372_DUALLINK_FORMAT(AjaVideoFormat);
			NewFormat.bIsSD = NTV2_IS_SD_VIDEO_FORMAT(AjaVideoFormat);
			NewFormat.bIsHD = NTV2_IS_HD_VIDEO_FORMAT(AjaVideoFormat);
			NewFormat.bIs2K = ::Is2KFormat(AjaVideoFormat);
			NewFormat.bIs4K = ::Is4KFormat(AjaVideoFormat);
			NewFormat.bIsValid = true;

			// fix bug in SDK, some formats have the wrong frame rate in dual link
			if (NewFormat.bIs372DualLink)
			{
				switch (InAjaVideoFormat)
				{
				case NTV2_FORMAT_1080p_5000_B:
				case NTV2_FORMAT_1080p_2K_5000_B:
					::GetFramesPerSecond(NTV2_FRAMERATE_5000, NewFormat.FrameRateNumerator, NewFormat.FrameRateDenominator);
					break;
				case NTV2_FORMAT_1080p_5994_B:
				case NTV2_FORMAT_1080p_2K_5994_B:
					::GetFramesPerSecond(NTV2_FRAMERATE_5994, NewFormat.FrameRateNumerator, NewFormat.FrameRateDenominator);
					break;
				case NTV2_FORMAT_1080p_6000_B:
				case NTV2_FORMAT_1080p_2K_6000_B:
					::GetFramesPerSecond(NTV2_FRAMERATE_6000, NewFormat.FrameRateNumerator, NewFormat.FrameRateDenominator);
					break;
				case NTV2_FORMAT_1080p_2K_4800_B:
				case NTV2_FORMAT_1080p_2K_4795_B:
					break;
				default:
					AJA_CHECK(false);
				}
			}

			return NewFormat;
		}
	}
}
