// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Helpers.h"

#include <vector>


namespace AJA
{
	namespace Private
	{
		/* VideoFormatsScanner definition
		*****************************************************************************/
		class VideoFormatsScanner
		{
		public:
			VideoFormatsScanner(int32_t InDeviceId);

			static AJAVideoFormats::VideoFormatDescriptor GetVideoFormat(FAJAVideoFormat InAjaVideoFormat);

			std::vector<AJAVideoFormats::VideoFormatDescriptor> FormatList;
		};
	}
}
