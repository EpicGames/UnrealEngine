// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <memory>
#include <string>
#include <vector>

#include "AjaCoreModule.h"
#include "Async/Future.h"
#include "Misc/Timecode.h"


class FRHITexture;

namespace UE::GPUTextureTransfer
{
	class ITextureTransfer;
}

namespace AJA
{
	/* Types provided from the interface
	*****************************************************************************/
	typedef void* FDeviceScanner;
	typedef void* FDeviceInfo;
	typedef void* FAJADevice;
	typedef uint32_t FAJAVideoFormat;

	/* Logging Callbacks
	*****************************************************************************/
	using LoggingCallbackPtr = void(*)(const TCHAR* Format, ...);

	/* Pixel formats supported
	*****************************************************************************/
	enum struct EPixelFormat
	{
		PF_8BIT_YCBCR,	// As Input/Output
		PF_8BIT_ARGB,	// As Input/Output
		PF_10BIT_RGB,	// As Input/Output
		PF_10BIT_YCBCR,	// As Input/Output
	};
	
	/* SDI transport type
	*****************************************************************************/
	enum class ETransportType
	{
		TT_SdiSingle,
		TT_SdiSingle4kTSI,
		TT_SdiDual,
		TT_SdiQuadSQ,
		TT_SdiQuadTSI,
		TT_Hdmi,
		TT_Hdmi4kTSI,
	};


	 /* Timecode
	 *****************************************************************************/
	struct AJACORE_API FTimecode
	{
		FTimecode();
		bool operator== (const FTimecode& Other) const;

		uint32_t Hours;
		uint32_t Minutes;
		uint32_t Seconds;
		uint32_t Frames;
		bool bDropFrame;
	};

	enum struct ETimecodeFormat
	{
		TCF_None,
		TCF_LTC,
		TCF_VITC1,
	};

	namespace Private
	{
		class AutoDetectChannel;
		class DeviceScanner;
		class InputChannel;
		class OutputChannel;
		class SyncChannel;
		class TimecodeChannel;
		class VideoFormatsScanner;
	}

	/* AJADeviceScanner definition
	*****************************************************************************/
	class AJACORE_API AJADeviceScanner
	{
	public:
		const static int32_t FormatedTextSize = 64;
		using FormatedTextType = TCHAR[FormatedTextSize];

		struct DeviceInfo
		{
			bool bIsSupported;

			bool bCanFrameStore1DoPlayback;
			bool bCanDoDualLink;
			bool bCanDo2K;
			bool bCanDo4K;
			bool bCanDo12GSdi;
			bool bCanDo12GRouting;
			bool bCanDoMultiFormat;
			bool bCanDoAlpha;
			bool bCanDo3GLevelConversion;
			bool bCanDoCustomAnc;
			bool bCanDoLtc;
			bool bCanDoLtcInRefPort;
			bool bCanDoTSI;
			bool bSupportPixelFormat8bitYCBCR;
			bool bSupportPixelFormat8bitARGB;
			bool bSupportPixelFormat10bitRGB;
			bool bSupportPixelFormat10bitYCBCR;

			uint32_t NumberOfLtcInput;
			uint32_t NumberOfLtcOutput;

			int32_t NumSdiInput;
			int32_t NumSdiOutput;
			int32_t NumHdmiInput;
			int32_t NumHdmiOutput;
		};

		AJADeviceScanner();
		~AJADeviceScanner();

		AJADeviceScanner(const AJADeviceScanner&) = delete;
		AJADeviceScanner& operator=(const AJADeviceScanner&) = delete;

		int32_t GetNumDevices() const;
		bool GetDeviceTextId(int32_t InDeviceIndex, FormatedTextType& OutTextId) const;
		bool GetDeviceInfo(int32_t InDeviceIndex, DeviceInfo& OutDeviceInfo) const;

	private:
		Private::DeviceScanner* Scanner;
	};

	/* AJAVideoFormats definition
	*****************************************************************************/
	struct AJACORE_API AJAVideoFormats
	{
		struct AJACORE_API VideoFormatDescriptor
		{
			VideoFormatDescriptor();

			FAJAVideoFormat VideoFormatIndex;
			uint32_t FrameRateNumerator;
			uint32_t FrameRateDenominator;
			uint32_t ResolutionWidth;
			uint32_t ResolutionHeight;
			bool bIsProgressiveStandard;
			bool bIsInterlacedStandard;
			bool bIsPsfStandard;
			bool bIsVideoFormatA;
			bool bIsVideoFormatB;
			bool bIs372DualLink;
			bool bIsSD;
			bool bIsHD;
			bool bIs2K;
			bool bIs4K;

			bool bIsValid;
		};

		AJAVideoFormats(int32_t InDeviceId);
		~AJAVideoFormats();

		AJAVideoFormats(const AJAVideoFormats&) = delete;
		AJAVideoFormats& operator=(const AJAVideoFormats&) = delete;

		int32_t GetNumSupportedFormat() const;
		VideoFormatDescriptor GetSupportedFormat(int32_t InIndex) const;
		static VideoFormatDescriptor GetVideoFormat(FAJAVideoFormat InVideoFormatIndex);
		static std::string VideoFormatToString(FAJAVideoFormat InVideoFormatIndex);
		static bool VideoFormatToString(FAJAVideoFormat InVideoFormatIndex, char* OutCStr, uint32_t MaxLen);

	private:
		Private::VideoFormatsScanner* Formats;
	};

	/* AJADeviceOptions definition
	*****************************************************************************/
	struct AJACORE_API AJADeviceOptions
	{
		AJADeviceOptions(uint32_t InChannelIndex)
			: DeviceIndex(InChannelIndex)
			, bWantMultiFormatMode(true)
		{}

		uint32_t DeviceIndex;
		bool bWantMultiFormatMode;
	};

	/* AJASyncChannel definition
	*****************************************************************************/
	struct AJACORE_API IAJASyncChannelCallbackInterface
	{
		IAJASyncChannelCallbackInterface();
		virtual ~IAJASyncChannelCallbackInterface();

		virtual void OnInitializationCompleted(bool bSucceed) = 0;
	};

	struct AJACORE_API AJASyncChannelOptions
	{
		AJASyncChannelOptions(const TCHAR* DebugName);

		IAJASyncChannelCallbackInterface* CallbackInterface;

		ETransportType TransportType;
		uint32_t ChannelIndex; // [1...x]
		FAJAVideoFormat VideoFormatIndex;
		ETimecodeFormat TimecodeFormat;
		bool bOutput; // port is output
		bool bWaitForFrameToBeReady; // port is input and we want to wait for the image to be sent to Unreal Engine before ticking
		bool bAutoDetectFormat;
		bool bStopOnCardTimeout = true;  // Stop sync channel whenever we timeout while waiting for a vertical interrupt from the card (Default: true)
	};

	class AJACORE_API AJASyncChannel
	{
	public:
		AJASyncChannel(AJASyncChannel&) = delete;
		AJASyncChannel& operator=(AJASyncChannel&) = delete;

		AJASyncChannel();
		~AJASyncChannel();

	public:
		bool Initialize(const AJADeviceOptions& InDevice, const AJASyncChannelOptions& InOption);
		void Uninitialize();

		// Only available if the initialization succeeded
		bool WaitForSync() const;
		bool GetTimecode(FTimecode& OutTimecode) const;
		bool GetSyncCount(uint32_t& OutCount) const;
		bool GetVideoFormat(FAJAVideoFormat& OutVideoFormat);

	private:
		Private::SyncChannel* Channel;
	};

	/* AJATimecodeChannel definition
	*****************************************************************************/
	struct AJACORE_API IAJATimecodeChannelCallbackInterface
	{
		IAJATimecodeChannelCallbackInterface();
		virtual ~IAJATimecodeChannelCallbackInterface();

		virtual void OnInitializationCompleted(bool bSucceed) = 0;
	};

	struct AJACORE_API AJATimecodeChannelOptions
	{
		AJATimecodeChannelOptions(const TCHAR* DebugName);

		IAJATimecodeChannelCallbackInterface* CallbackInterface;

		bool bUseDedicatedPin;

		//Timecode read from dedicated pin
		bool bReadTimecodeFromReferenceIn;
		uint32_t LTCSourceIndex; //[1...x]
		uint32_t LTCFrameRateNumerator;
		uint32_t LTCFrameRateDenominator;

		//Timecode read from input channels
		ETransportType TransportType;
		uint32_t ChannelIndex; // [1...x]
		FAJAVideoFormat VideoFormatIndex;
		ETimecodeFormat TimecodeFormat;
		bool bAutoDetectFormat;
	};

	class AJACORE_API AJATimecodeChannel
	{
	public:
		AJATimecodeChannel(AJATimecodeChannel&) = delete;
		AJATimecodeChannel& operator=(AJATimecodeChannel&) = delete;

		AJATimecodeChannel();
		~AJATimecodeChannel();

	public:
		bool Initialize(const AJADeviceOptions& InDevice, const AJATimecodeChannelOptions& InOption);
		void Uninitialize();

		// Only available if the initialization succeeded
		bool GetTimecode(FTimecode& OutTimecode) const;

	private:
		Private::TimecodeChannel* Channel;
	};
	
	
	/**
	 * HDR Transfer function.
	 */
	enum class EAjaHDRMetadataEOTF : uint8
	{
		SDR,
		HLG,
		PQ,
		Unspecified
	};
	
	/**
	 * HDR Color Gamut.
	 */
	enum class EAjaHDRMetadataGamut : uint8
	{
		Rec709,
		Rec2020,
		Invalid
	};

	/**
	 * HDR Luminance.
	 */
	enum class EAjaHDRMetadataLuminance : uint8
	{
		YCbCr,
		ICtCp
	};

	/**
	 * Set of metadata describing a HDR video signal.
	 */
	struct AJACORE_API FAjaHDROptions
	{
		/** Transfer function to use for converting the video signal to an optical signal. */
		EAjaHDRMetadataEOTF EOTF = EAjaHDRMetadataEOTF::SDR;

		/** The color gamut of the video signal. */
		EAjaHDRMetadataGamut Gamut = EAjaHDRMetadataGamut::Rec709;

		/** Color representation format of the video signal. */
		EAjaHDRMetadataLuminance Luminance = EAjaHDRMetadataLuminance::YCbCr;
	};

	/* AJAInputFrameData definition
	*****************************************************************************/
	struct AJACORE_API AJAInputFrameData
	{
		AJAInputFrameData();

		FTimecode Timecode;
		uint32_t FramesDropped; // frame dropped by the AJA
	};

	struct AJACORE_API AJAOutputFrameData : AJAInputFrameData
	{
		AJAOutputFrameData();

		uint32_t FramesLost; // frame ready by the game but not sent to AJA
	};

	struct AJACORE_API AJAAncillaryFrameData
	{
		AJAAncillaryFrameData();

		uint8_t* AncBuffer;
		uint32_t AncBufferSize;
		uint8_t* AncF2Buffer;
		uint32_t AncF2BufferSize;
	};

	struct AJACORE_API AJAAudioFrameData
	{
		AJAAudioFrameData();

		uint8_t* AudioBuffer;
		uint32_t AudioBufferSize;
		uint32_t NumChannels;
		uint32_t AudioRate;
		uint32_t NumSamples;
	};

	struct AJACORE_API AJAVideoFrameData
	{
		AJAVideoFrameData();
		FAJAVideoFormat VideoFormatIndex;
		uint8_t* VideoBuffer;
		uint32_t VideoBufferSize;
		uint32_t Stride;
		uint32_t Width;
		uint32_t Height;
		EPixelFormat PixelFormat;
		bool bIsProgressivePicture;
		FAjaHDROptions HDROptions;
	};

	struct AJACORE_API AJARequestInputBufferData
	{
		AJARequestInputBufferData();
		bool bIsProgressivePicture;
		uint32_t AncBufferSize;
		uint32_t AncF2BufferSize;
		uint32_t AudioBufferSize;
		uint32_t VideoBufferSize;
	};

	struct AJACORE_API AJARequestedInputBufferData
	{
		AJARequestedInputBufferData();
		uint8_t* AncBuffer;
		uint8_t* AncF2Buffer;
		uint8_t* AudioBuffer;
		uint8_t* VideoBuffer;
	};

	struct AJACORE_API IAJAInputOutputChannelCallbackInterface : IAJASyncChannelCallbackInterface
	{
		IAJAInputOutputChannelCallbackInterface();

		virtual bool OnRequestInputBuffer(const AJARequestInputBufferData& RequestBuffer, AJARequestedInputBufferData& OutRequestedBuffer) = 0;
		virtual bool OnInputFrameReceived(const AJAInputFrameData& InFrameData, const AJAAncillaryFrameData& InAncillaryFrame, const AJAAudioFrameData& InAudioFrame, const AJAVideoFrameData& InVideoFrame) = 0;
		virtual void OnOutputFrameStarted() { }
		virtual bool OnOutputFrameCopied(const AJAOutputFrameData& InFrameData) = 0;
		virtual void OnCompletion(bool bSucceed) = 0;
		virtual void OnFormatChange(FAJAVideoFormat VideoFormat) {}
	};

	/* AJAInputOutputChannelOptions definition
	*****************************************************************************/
	enum class EAJAReferenceType
	{
		EAJA_REFERENCETYPE_EXTERNAL,
		EAJA_REFERENCETYPE_FREERUN,
		EAJA_REFERENCETYPE_INPUT,
	};

	struct AJACORE_API AJAInputOutputChannelOptions
	{
		AJAInputOutputChannelOptions(const TCHAR* DebugName, uint32_t InChannelIndex);

		IAJAInputOutputChannelCallbackInterface* CallbackInterface;

		uint32_t NumberOfAudioChannel;
		ETransportType TransportType;
		uint32_t ChannelIndex; // [1...x]
		uint32_t SynchronizeChannelIndex; // [1...x]
		uint32_t KeyChannelIndex; // [1...x] for output
		uint32_t OutputNumberOfBuffers; // [1...x] supported but not suggested (min of 2 is suggested)
		FAJAVideoFormat VideoFormatIndex;
		EPixelFormat PixelFormat;
		FAjaHDROptions HDROptions;
		ETimecodeFormat TimecodeFormat;
		EAJAReferenceType OutputReferenceType;
		uint32_t BurnTimecodePercentY;

		union
		{
			struct 
			{
				uint32_t bUseAutoCirculating : 1;
				uint32_t bOutput : 1; // port is output
				uint32_t bUseKey : 1; // output will also sent the key on OutputKeyPortIndex
				uint32_t bOutputInterlacedFieldsTimecodeNeedToMatch : 1; // when trying to find the odd field that correspond to the even field, the 2 timecode need to match
				uint32_t bOutputInterlaceAsProgressive : 1; // whether to treat outgoing interlace frames as progressive.
				uint32_t bOutputInterlaceOnEvenFrames : 1; // whether to force output interlace on even timecode frames.
				uint32_t bUseAncillary : 1; // enable ANC system
				uint32_t bUseAudio : 1; // enable audio input/output
				uint32_t bUseVideo : 1; // enable video input/output
				uint32_t bBurnTimecode : 1; // burn the timecode to the input or output image
				uint32_t bDisplayWarningIfDropFrames : 1;
				uint32_t bConvertOutputLevelAToB : 1; // enable video output 3G level A to convert it to 3G level B
				uint32_t bTEST_OutputInterlaced : 1; // when outputting, warn if the field 1 and field 2 are the same color. It except one of the line to be white and the other line to be not white
				uint32_t bUseGPUDMA : 1; // Whether to use GPU Direct when outputting.
				uint32_t bAutoDetectFormat : 1; // Whether to autodetect format on input.
				uint32_t bDirectlyWriteAudio : 1; // Experimental mode to set audio from the audio thread directly.
				uint32_t bStopOnTimeout : 1; // Stop input/output whenever we timeout while waiting for a vertical interrupt from the card.
			};
			uint32_t Options;
		};
	};

	/* AJAInputChannel definition
	*****************************************************************************/
	class AJACORE_API AJAInputChannel
	{
	public:
		AJAInputChannel(AJAInputChannel&) = delete;
		AJAInputChannel& operator=(AJAInputChannel&) = delete;

		AJAInputChannel();
		virtual ~AJAInputChannel();

	public:
		bool Initialize(const AJADeviceOptions& InDevice, const AJAInputOutputChannelOptions& Options);
		void Uninitialize();

		// Only available if the initialization succeeded
		uint32_t GetFrameDropCount() const;
		const AJAInputOutputChannelOptions& GetOptions() const;
		const AJADeviceOptions& GetDeviceOptions() const;

	private:
		Private::InputChannel* Channel;
	};

	/* AJAOutputFrameBufferData definition
	*****************************************************************************/
	struct AJACORE_API AJAOutputFrameBufferData
	{
		AJAOutputFrameBufferData();

		static const uint32_t InvalidFrameIdentifier;

		FTimecode Timecode;
		uint32_t FrameIdentifier;
		bool bEvenFrame;
	};

	/* AJAOutputChannel definition
	*****************************************************************************/
	class AJACORE_API AJAOutputChannel
	{
	public:
		AJAOutputChannel(AJAOutputChannel&) = delete;
		AJAOutputChannel& operator=(AJAOutputChannel&) = delete;

		AJAOutputChannel();
		virtual ~AJAOutputChannel();

	public:
		bool Initialize(const AJADeviceOptions& InDevice, const AJAInputOutputChannelOptions& Options);
	
		/** 
		 * Clean up card connections for this channel. (Asynchronous)
		 * @param Promise Used to inform when the uninitialization is complete.
		 */
		void Uninitialize(TPromise<void> Promise = {});

		// Set a new buffer that will be copied to the AJA.
		bool SetAncillaryFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* AncillaryBuffer, uint32_t AncillaryBufferSize);
		bool SetAudioFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* AudioBuffer, uint32_t AudioBufferSize);
		bool SetVideoFrameData(const AJAOutputFrameBufferData& InFrameData, uint8_t* VideoBuffer, uint32_t VideoBufferSize);
		bool SetVideoFrameData(const AJAOutputFrameBufferData& InFrameData, FRHITexture* RHITexture);
		bool DMAWriteAudio(const uint8* Buffer, int32 BufferSize);

		// Return whether to capture or discard a frame based on its timecode and frame number.
		bool ShouldCaptureThisFrame(::FTimecode Timecode, uint32 SourceFrameNumber);

		bool GetOutputDimension(uint32_t& OutWidth, uint32_t& OutHeight) const;
		int32_t GetNumAudioSamplesPerFrame(const AJAOutputFrameBufferData& InFrameData) const;

	private:
		Private::OutputChannel* Channel;
	};


	/* AJAAutoDetectChannelCallbackInterface definition
	*****************************************************************************/
	struct AJACORE_API IAJAAutoDetectCallbackInterface 
	{
		IAJAAutoDetectCallbackInterface();
		virtual ~IAJAAutoDetectCallbackInterface();

		virtual void OnCompletion(bool bSucceed) = 0;
	};

	/* AJAAutoDetectChannel definition
	*****************************************************************************/
	class AJACORE_API AJAAutoDetectChannel
	{
	public:
		struct AutoDetectChannelData
		{
			AutoDetectChannelData();

			FAJAVideoFormat DetectedVideoFormat;
			uint32_t DeviceIndex; // [0...x]
			uint32_t ChannelIndex; // [1...x]
			ETimecodeFormat TimecodeFormat;
		};

	public:
		AJAAutoDetectChannel(AJAAutoDetectChannel&) = delete;
		AJAAutoDetectChannel& operator=(AJAAutoDetectChannel&) = delete;

		AJAAutoDetectChannel();
		virtual ~AJAAutoDetectChannel();

	public:
		bool Initialize(IAJAAutoDetectCallbackInterface* InCallbackInterface);
		void Uninitialize();

		int32_t GetNumOfChannelData() const;
		AutoDetectChannelData GetChannelData(int32_t Index) const;

	private:
		Private::AutoDetectChannel* AutoChannel;
	};
}
