// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	using System.IO;

	public class AjaMedia : ModuleRules
	{
		public AjaMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AjaCore",
					"Core",
					"CoreUObject",
					"Engine",
					"MediaIOCore",
					"MediaUtils",
					"OpenColorIO",
					"Projects",
					"TimeManagement",
					"RenderCore",
					"RHI",
					"SlateCore"
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"MediaAssets",
				});
		}
	}
}
