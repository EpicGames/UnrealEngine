// Copyright Epic Games, Inc. All Rights Reserved.

#include "Aja/Aja.h"
#include "AjaMediaPrivate.h"

#include "Misc/FrameRate.h"
#include "Interfaces/IPluginManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"

 //~ Static initialization
 //--------------------------------------------------------------------
bool FAja::bCanForceAJAUsage = false;

//~ Initialization functions implementation
//--------------------------------------------------------------------
bool FAja::Initialize()
{
	// todo: migrate this to aja module startup
#if AJAMEDIA_DLL_PLATFORM
	bCanForceAJAUsage = FParse::Param(FCommandLine::Get(), TEXT("forceajausage"));
	return true;
#else
	return false;
#endif
}

bool FAja::IsInitialized()
{
	return true;
}

void FAja::Shutdown()
{
}

//~ Conversion functions implementation
//--------------------------------------------------------------------

FTimecode FAja::ConvertAJATimecode2Timecode(const AJA::FTimecode& InTimecode, const FFrameRate& InFPS)
{
	return FTimecode(InTimecode.Hours, InTimecode.Minutes, InTimecode.Seconds, InTimecode.Frames, InTimecode.bDropFrame);

}



