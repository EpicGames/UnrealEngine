// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/App.h"
#include "Misc/Timecode.h"
#include "Misc/Timespan.h"

namespace AJA
{
	struct FTimecode;
}
struct FFrameRate;

class FAja
{
public:
	static bool Initialize();
	static bool IsInitialized();
	static void Shutdown();

	// Helpers
	static FTimecode ConvertAJATimecode2Timecode(const AJA::FTimecode& InTimecode, const FFrameRate& InFPS);

	static bool CanUseAJACard() { return (FApp::CanEverRender() || bCanForceAJAUsage); }

private:
	static bool bCanForceAJAUsage;
};
