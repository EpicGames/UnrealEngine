// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaDeviceProvider.h"

#include "Algo/Sort.h"
#include "Aja/Aja.h"
#include "AjaMediaPrivate.h"

#include "Async/Async.h"
#include "CommonFrameRates.h"

#define LOCTEXT_NAMESPACE "AjaDeviceProvider"


//~ FAjaMediaTimecodeConfiguration implementation
//--------------------------------------------------------------------
FAjaMediaTimecodeConfiguration::FAjaMediaTimecodeConfiguration()
	: TimecodeFormat(EMediaIOTimecodeFormat::LTC)
{
}

FAjaMediaTimecodeConfiguration FAjaMediaTimecodeConfiguration::GetDefault()
{
	FAjaMediaTimecodeConfiguration Result;
	Result.MediaConfiguration = FAjaDeviceProvider().GetDefaultConfiguration();
	return Result;
}

bool FAjaMediaTimecodeConfiguration::IsValid() const
{
	return TimecodeFormat != EMediaIOTimecodeFormat::None && MediaConfiguration.IsValid();
}

bool FAjaMediaTimecodeConfiguration::operator== (const FAjaMediaTimecodeConfiguration& Other) const
{
	return MediaConfiguration == Other.MediaConfiguration
		&& TimecodeFormat == Other.TimecodeFormat;
}

FText FAjaMediaTimecodeConfiguration::ToText() const
{
	if (IsValid())
	{
		FText Timecode = LOCTEXT("Invalid", "<Invalid>");
		switch(TimecodeFormat)
		{
		case EMediaIOTimecodeFormat::LTC:
			Timecode = LOCTEXT("LTCLabel", "LTC");
			break;
		case EMediaIOTimecodeFormat::VITC:
			Timecode = LOCTEXT("VITCLabel", "VITC");
			break;
		}

		return FText::Format(LOCTEXT("MediaTimecodeConfigurationRefText", "{0}/{1}")
			, FAjaDeviceProvider().ToText(MediaConfiguration)
			, Timecode
		);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}

//~ FAjaMediaTimecodeReference implementation
//--------------------------------------------------------------------
FAjaMediaTimecodeReference::FAjaMediaTimecodeReference()
	: LtcIndex(-1)
	, LtcFrameRate(30, 1)
{
}

FAjaMediaTimecodeReference FAjaMediaTimecodeReference::GetDefault()
{
	FAjaMediaTimecodeReference Reference;
	Reference.Device.DeviceIdentifier = 1;
	Reference.LtcIndex = 1;
	Reference.LtcFrameRate = FFrameRate(30, 1);
	return Reference;
}

bool FAjaMediaTimecodeReference::IsValid() const
{
	return LtcIndex > 0 && Device.IsValid();
}

bool FAjaMediaTimecodeReference::operator== (const FAjaMediaTimecodeReference& Other) const
{
	return LtcIndex == Other.LtcIndex
		&& Device == Other.Device
		&& LtcFrameRate == Other.LtcFrameRate;
}

FText FAjaMediaTimecodeReference::ToText() const
{
	if (IsValid())
	{
		return FText::Format(LOCTEXT("MediaTimecodeRefText", "Ref[{0}] {1} [device{2}/{3}]")
			, FText::AsNumber(LtcIndex)
			, FText::FromName(Device.DeviceName)
			, FText::AsNumber(Device.DeviceIdentifier)
			, LtcFrameRate.ToPrettyText()
		);
	}

	return LOCTEXT("Invalid", "<Invalid>");
}

//~ AjaDeviceProvider namespace
//--------------------------------------------------------------------
namespace AjaDeviceProvider
{
	FMediaIOMode ToMediaMode(const AJA::AJAVideoFormats::VideoFormatDescriptor& InDescriptor)
	{
		FMediaIOMode MediaMode;
		MediaMode.Resolution = FIntPoint(InDescriptor.ResolutionWidth, InDescriptor.ResolutionHeight);
		MediaMode.Standard = EMediaIOStandardType::Progressive;
		if (InDescriptor.bIsInterlacedStandard)
		{
			MediaMode.Standard = EMediaIOStandardType::Interlaced;
		}
		else if (InDescriptor.bIsPsfStandard)
		{
			MediaMode.Standard = EMediaIOStandardType::ProgressiveSegmentedFrame;
		}
		MediaMode.FrameRate = FFrameRate(InDescriptor.FrameRateNumerator, InDescriptor.FrameRateDenominator);
		MediaMode.DeviceModeIdentifier = InDescriptor.VideoFormatIndex;

		if (InDescriptor.bIsInterlacedStandard)
		{
			MediaMode.FrameRate.Numerator *= 2;
		}

		return MediaMode;
	}

	bool IsVideoFormatValid(const AJA::AJAVideoFormats::VideoFormatDescriptor& InDescriptor)
	{
		if (!InDescriptor.bIsValid)
		{
			return false;
		}
		if (InDescriptor.bIsSD)
		{
			return false;
		}
		if (InDescriptor.bIsVideoFormatB && !InDescriptor.bIs372DualLink)
		{
			return false;
		}
		return true;
	}
}

//~ FAJAAutoDetectChannelCallback implementation
//--------------------------------------------------------------------
class FAJAAutoDetectChannelCallback : public AJA::IAJAAutoDetectCallbackInterface
{
public:
	TSharedPtr<AJA::AJAAutoDetectChannel, ESPMode::ThreadSafe> AutoDetectChannel;
	FAjaDeviceProvider::FOnConfigurationAutoDetected OnAutoDetected;

	FAJAAutoDetectChannelCallback(FAjaDeviceProvider::FOnConfigurationAutoDetected InOnAutoDetected)
		: OnAutoDetected(InOnAutoDetected)
	{
		AutoDetectChannel = MakeShared<AJA::AJAAutoDetectChannel, ESPMode::ThreadSafe>();
		AutoDetectChannel->Initialize(this);
	}

	~FAJAAutoDetectChannelCallback()
	{
		AutoDetectChannel->Uninitialize();
	}

	virtual void OnCompletion(bool bSucceed) override
	{
		TWeakPtr<AJA::AJAAutoDetectChannel, ESPMode::ThreadSafe> InWeakAutoDetectChannel = AutoDetectChannel;
		FAjaDeviceProvider::FOnConfigurationAutoDetected InOnAutoDetected = OnAutoDetected;

		AsyncTask(ENamedThreads::GameThread, [InWeakAutoDetectChannel, InOnAutoDetected]()
		{
			if (TSharedPtr<AJA::AJAAutoDetectChannel, ESPMode::ThreadSafe> AutoDetectChannelPin = InWeakAutoDetectChannel.Pin())
			{
				TArray<FAjaDeviceProvider::FMediaIOConfigurationWithTimecodeFormat> Configurations;
				int32 NumberOfChannels = AutoDetectChannelPin->GetNumOfChannelData();
				for (int32 Index = 0; Index < NumberOfChannels; ++Index)
				{
					AJA::AJAAutoDetectChannel::AutoDetectChannelData Data = AutoDetectChannelPin->GetChannelData(Index);

					AJA::AJADeviceScanner DeviceScanner;
					AJA::AJADeviceScanner::DeviceInfo DeviceInfo;
					if (!DeviceScanner.GetDeviceInfo(Data.DeviceIndex, DeviceInfo))
					{
						continue;
					}

					if (!DeviceInfo.bIsSupported)
					{
						continue;
					}

					TCHAR DeviceNameBuffer[AJA::AJADeviceScanner::FormatedTextSize];
					if (!DeviceScanner.GetDeviceTextId(Data.DeviceIndex, DeviceNameBuffer))
					{
						continue;
					}

					FAjaDeviceProvider::FMediaIOConfigurationWithTimecodeFormat NewResult;
					NewResult.Configuration.bIsInput = true;
					NewResult.Configuration.MediaConnection.Device.DeviceIdentifier = Data.DeviceIndex;
					NewResult.Configuration.MediaConnection.Device.DeviceName = DeviceNameBuffer;
					NewResult.Configuration.MediaConnection.Protocol = FAjaDeviceProvider::GetProtocolName();
					NewResult.Configuration.MediaConnection.PortIdentifier = Data.ChannelIndex + 1;
					NewResult.Configuration.MediaConnection.TransportType = EMediaIOTransportType::SingleLink;

					const AJA::AJAVideoFormats::VideoFormatDescriptor Descriptor = AJA::AJAVideoFormats::GetVideoFormat(Data.DetectedVideoFormat);
					if (!AjaDeviceProvider::IsVideoFormatValid(Descriptor))
					{
						continue;
					}

					NewResult.Configuration.MediaMode = AjaDeviceProvider::ToMediaMode(Descriptor);
					switch (Data.TimecodeFormat)
					{
						case AJA::ETimecodeFormat::TCF_LTC:
							NewResult.TimecodeFormat = EMediaIOTimecodeFormat::LTC;
							break;
						case AJA::ETimecodeFormat::TCF_VITC1:
							NewResult.TimecodeFormat = EMediaIOTimecodeFormat::VITC;
							break;
						case AJA::ETimecodeFormat::TCF_None:
						default:
							NewResult.TimecodeFormat = EMediaIOTimecodeFormat::None;
							break;
					}

					Configurations.Add(NewResult);
				}

				InOnAutoDetected.ExecuteIfBound(Configurations);
				AutoDetectChannelPin->Uninitialize();
			}
		});
	}
};

//~ FAjaDeviceProvider implementation
//--------------------------------------------------------------------
FAjaDeviceProvider::FAjaDeviceProvider()
{
}


FAjaDeviceProvider::~FAjaDeviceProvider()
{
}

FName FAjaDeviceProvider::GetProviderName()
{
	static FName NAME_Provider = "aja";
	return NAME_Provider;
}

FName FAjaDeviceProvider::GetProtocolName()
{
	static FName NAME_Protocol = "aja"; // also defined in FAjaMediaFactoryModule
	return NAME_Protocol;
}

FName FAjaDeviceProvider::GetFName()
{
	return GetProviderName();
}

bool FAjaDeviceProvider::CanDeviceDoAlpha(const FMediaIODevice& InDevice) const
{
	if (!FAja::IsInitialized() || !InDevice.IsValid() || !FAja::CanUseAJACard())
	{
		return false;
	}

	AJA::AJADeviceScanner DeviceScanner;
	AJA::AJADeviceScanner::DeviceInfo DeviceInfo;
	if (!DeviceScanner.GetDeviceInfo(InDevice.DeviceIdentifier, DeviceInfo))
	{
		return false;
	}

	if (!DeviceInfo.bIsSupported)
	{
		return false;
	}

	return DeviceInfo.bCanDoAlpha;
}

void FAjaDeviceProvider::AutoDetectConfiguration(FOnConfigurationAutoDetected OnAutoDetected)
{
	if (!FAja::IsInitialized() || !FAja::CanUseAJACard())
	{
		TArray<FMediaIOConfigurationWithTimecodeFormat> Results;
		OnAutoDetected.ExecuteIfBound(Results);
		return;
	}

	if (OnAutoDetected.IsBound())
	{
		AutoDetectCallback = MakeUnique<FAJAAutoDetectChannelCallback>(OnAutoDetected);
	}
}

void FAjaDeviceProvider::EndAutoDetectConfiguration()
{
	AutoDetectCallback.Reset();
}

TArray<FMediaIOConnection> FAjaDeviceProvider::GetConnections() const
{
	TArray<FMediaIOConnection> Results;
	if (!FAja::IsInitialized() || !FAja::CanUseAJACard())
	{
		return Results;
	}

	AJA::AJADeviceScanner DeviceScanner;
	const int32 NumDevices = DeviceScanner.GetNumDevices();
	for (int32 DeviceIndex = 0; DeviceIndex < NumDevices; ++DeviceIndex)
	{
		AJA::AJADeviceScanner::DeviceInfo DeviceInfo;
		if (!DeviceScanner.GetDeviceInfo(DeviceIndex, DeviceInfo))
		{
			continue;
		}

		if (!DeviceInfo.bIsSupported)
		{
			continue;
		}

		TCHAR DeviceNameBuffer[AJA::AJADeviceScanner::FormatedTextSize];
		if (DeviceScanner.GetDeviceTextId(DeviceIndex, DeviceNameBuffer))
		{
			FMediaIOConnection Connection;
			Connection.Device.DeviceName = DeviceNameBuffer;
			Connection.Device.DeviceIdentifier = DeviceIndex;
			Connection.Protocol = GetProtocolName();

			Connection.TransportType = EMediaIOTransportType::SingleLink;
			for (int32 Input = 0; Input < DeviceInfo.NumSdiInput; ++Input)
			{
				Connection.PortIdentifier = Input + 1;
				Results.Add(Connection);
			}

			Connection.TransportType = EMediaIOTransportType::HDMI;
			for (int32 Input = 0; Input < DeviceInfo.NumHdmiInput; ++Input)
			{
				Connection.PortIdentifier = Input + 1;
				Results.Add(Connection);
			}
		}
	}

	return Results;
}

TArray<FMediaIOConfiguration> FAjaDeviceProvider::GetConfigurations() const
{
	return GetConfigurations(true, true);
}

TArray<FMediaIOConfiguration> FAjaDeviceProvider::GetConfigurations(bool bAllowInput, bool bAllowOutput) const
{
	const int32 MaxNumberOfChannel = 8;

	TArray<FMediaIOConfiguration> Results;
	if (!FAja::IsInitialized() || !FAja::CanUseAJACard())
	{
		return Results;
	}

	AJA::AJADeviceScanner DeviceScanner;
	int32 NumDevices = DeviceScanner.GetNumDevices();
	for (int32 DeviceIndex = 0; DeviceIndex < NumDevices; ++DeviceIndex)
	{
		TCHAR DeviceNameBuffer[AJA::AJADeviceScanner::FormatedTextSize];
		if (DeviceScanner.GetDeviceTextId(DeviceIndex, DeviceNameBuffer))
		{
			AJA::AJADeviceScanner::DeviceInfo DeviceInfo;
			if (DeviceScanner.GetDeviceInfo(DeviceIndex, DeviceInfo))
			{
				if (DeviceInfo.bIsSupported)
				{
					const bool bDeviceHasInput = DeviceInfo.NumSdiInput > 0 || DeviceInfo.NumHdmiInput > 0;
					if (bAllowInput && !bDeviceHasInput)
					{
						continue;
					}

					const bool bDeviceHasOutput = DeviceInfo.NumSdiOutput > 0 || DeviceInfo.NumHdmiOutput > 0;
					if (bAllowOutput && !bDeviceHasOutput)
					{
						continue;
					}

					const int32 SdiInputCount = FMath::Min(DeviceInfo.NumSdiInput, MaxNumberOfChannel);
					const int32 SdiOutputCount = FMath::Min(DeviceInfo.NumSdiOutput, MaxNumberOfChannel);
					const int32 HdmiInputCount = FMath::Min(DeviceInfo.NumHdmiInput, MaxNumberOfChannel);
					const int32 HdmiOutputCount = FMath::Min(DeviceInfo.NumHdmiOutput, MaxNumberOfChannel);

					AJA::AJAVideoFormats FrameFormats(DeviceIndex);
					const int32 NumSupportedFormat = FrameFormats.GetNumSupportedFormat();

					FMediaIOConfiguration MediaConfiguration;
					MediaConfiguration.MediaConnection.Device.DeviceIdentifier = DeviceIndex;
					MediaConfiguration.MediaConnection.Device.DeviceName = FName(DeviceNameBuffer);
					MediaConfiguration.MediaConnection.Protocol = FAjaDeviceProvider::GetProtocolName();

					for (int32 InputOutputLoop = 0; InputOutputLoop < 2; ++InputOutputLoop)
					{
						// Build input or output
						MediaConfiguration.bIsInput = (InputOutputLoop == 0);
						if (!bAllowInput && MediaConfiguration.bIsInput)
						{
							continue;
						}
						if (!bAllowOutput && !MediaConfiguration.bIsInput)
						{
							continue;
						}

						const int32 SdiPortCount = MediaConfiguration.bIsInput ? SdiInputCount : SdiOutputCount;
						if (SdiPortCount > 0)
						{
							for (int32 FormatIndex = 0; FormatIndex < NumSupportedFormat; ++FormatIndex)
							{
								const AJA::AJAVideoFormats::VideoFormatDescriptor Descriptor = FrameFormats.GetSupportedFormat(FormatIndex);
								if (!AjaDeviceProvider::IsVideoFormatValid(Descriptor))
								{
									continue;
								}

								MediaConfiguration.MediaMode = AjaDeviceProvider::ToMediaMode(Descriptor);
								MediaConfiguration.MediaConnection.QuadTransportType = EMediaIOQuadLinkTransportType::SquareDivision;

								const bool bRequiredMoreThan3G = Descriptor.bIs4K || Descriptor.bIs2K;

								if (Descriptor.bIs372DualLink && DeviceInfo.bCanDoDualLink)
								{
									MediaConfiguration.MediaConnection.TransportType = EMediaIOTransportType::DualLink;
									for (int32 SourceIndex = 0; SourceIndex < SdiPortCount/2; ++SourceIndex)
									{
										MediaConfiguration.MediaConnection.PortIdentifier = (SourceIndex*2) + 1;
										Results.Add(MediaConfiguration);
									}
								}

								if (Descriptor.bIs4K && SdiPortCount >= 4 && DeviceInfo.bCanDo4K)
								{
									MediaConfiguration.MediaConnection.TransportType = EMediaIOTransportType::QuadLink;
									for (int32 SourceIndex = 0; SourceIndex < SdiPortCount/4; ++SourceIndex)
									{
										MediaConfiguration.MediaConnection.QuadTransportType = EMediaIOQuadLinkTransportType::SquareDivision;
										MediaConfiguration.MediaConnection.PortIdentifier = (SourceIndex*4) + 1;
										Results.Add(MediaConfiguration);

										if (DeviceInfo.bCanDoTSI)
										{
											MediaConfiguration.MediaConnection.QuadTransportType = EMediaIOQuadLinkTransportType::TwoSampleInterleave;
											Results.Add(MediaConfiguration);
										}
									}
								}

								if (!Descriptor.bIsVideoFormatB)
								{
									if ((DeviceInfo.bCanDo12GRouting && bRequiredMoreThan3G) || !bRequiredMoreThan3G)
									{
										MediaConfiguration.MediaConnection.TransportType = EMediaIOTransportType::SingleLink;
										for (int32 SourceIndex = 0; SourceIndex < SdiPortCount; ++SourceIndex)
										{
											MediaConfiguration.MediaConnection.PortIdentifier = SourceIndex + 1;
											Results.Add(MediaConfiguration);
										}
									}
									else if (DeviceInfo.bCanDo12GSdi && bRequiredMoreThan3G)
									{
										MediaConfiguration.MediaConnection.TransportType = EMediaIOTransportType::SingleLink;
										// in single TSI, we only support input 1, 3, 5, 7
										// otherwise it will be single-quad
										constexpr int32 Increment = 2;
										const int32 MaxSourceIndex = (MediaConfiguration.bIsInput && Descriptor.bIs4K) ? 1 : SdiPortCount / Increment; //Corner case for Kona 5 retail firmware. Can do 4k input only on SDI 1.
										for (int32 SourceIndex = 0; SourceIndex < MaxSourceIndex; ++SourceIndex)
										{
											MediaConfiguration.MediaConnection.PortIdentifier = (SourceIndex*Increment) + 1;
											Results.Add(MediaConfiguration);
										}
									}
								}
							}
						}

						const int32 HdmiPortCount = MediaConfiguration.bIsInput ? HdmiInputCount : HdmiOutputCount;
						if (HdmiPortCount > 0)
						{
							for (int32 FormatIndex = 0; FormatIndex < NumSupportedFormat; ++FormatIndex)
							{
								const AJA::AJAVideoFormats::VideoFormatDescriptor Descriptor = FrameFormats.GetSupportedFormat(FormatIndex);
								if (!AjaDeviceProvider::IsVideoFormatValid(Descriptor))
								{
									continue;
								}

								if (Descriptor.bIsVideoFormatB)
								{
									continue;
								}

								MediaConfiguration.MediaMode = AjaDeviceProvider::ToMediaMode(Descriptor);
								MediaConfiguration.MediaConnection.QuadTransportType = EMediaIOQuadLinkTransportType::SquareDivision;

								MediaConfiguration.MediaConnection.TransportType = EMediaIOTransportType::HDMI;
								for (int32 SourceIndex = 0; SourceIndex < HdmiPortCount; ++SourceIndex)
								{
									MediaConfiguration.MediaConnection.PortIdentifier = SourceIndex + 1;
									Results.Add(MediaConfiguration);
								}
							}
						}
					}
				}
			}
		}
	}

	return Results;
}

TArray<FMediaIOInputConfiguration> FAjaDeviceProvider::GetInputConfigurations() const
{
	TArray<FMediaIOInputConfiguration> Results;
	TArray<FMediaIOConfiguration> InputConfigurations = GetConfigurations(true, false);
	TArray<FMediaIOConnection> OtherSources = GetConnections();

	FMediaIOInputConfiguration DefaultInputConfiguration = GetDefaultInputConfiguration();
	Results.Reset(InputConfigurations.Num() * 2);

	int32 LastDeviceIndex = INDEX_NONE;
	bool bCanDoKeyAndFill = false;

	for (const FMediaIOConfiguration& InputConfiguration : InputConfigurations)
	{
		// Update the Device Info
		if (InputConfiguration.MediaConnection.Device.DeviceIdentifier != LastDeviceIndex)
		{
			LastDeviceIndex = InputConfiguration.MediaConnection.Device.DeviceIdentifier;
			bCanDoKeyAndFill = CanDeviceDoAlpha(InputConfiguration.MediaConnection.Device);
		}

		DefaultInputConfiguration.MediaConfiguration = InputConfiguration;

		// Build the list for fill
		DefaultInputConfiguration.InputType = EMediaIOInputType::Fill;
		Results.Add(DefaultInputConfiguration);

		// Add all output port for key
		if (bCanDoKeyAndFill)
		{
			DefaultInputConfiguration.InputType = EMediaIOInputType::FillAndKey;
			for (const FMediaIOConnection& InputPort : OtherSources)
			{
				if (InputPort.Device == InputConfiguration.MediaConnection.Device && InputPort.TransportType == InputConfiguration.MediaConnection.TransportType && InputPort.PortIdentifier != InputConfiguration.MediaConnection.PortIdentifier)
				{
					if (InputPort.TransportType != EMediaIOTransportType::QuadLink || InputPort.QuadTransportType == InputConfiguration.MediaConnection.QuadTransportType)
					{
						DefaultInputConfiguration.KeyPortIdentifier = InputPort.PortIdentifier;
						Results.Add(DefaultInputConfiguration);
					}
				}
			}
		}
	}

	Algo::SortBy(Results, [](const FMediaIOInputConfiguration& Config){ return Config.MediaConfiguration.MediaConnection.Device.DeviceName; }, FNameLexicalLess());

	return Results;
}

TArray<FMediaIOOutputConfiguration> FAjaDeviceProvider::GetOutputConfigurations() const
{
	TArray<FMediaIOOutputConfiguration> Results;
	TArray<FMediaIOConfiguration> OutputConfigurations = GetConfigurations(false, true);
	TArray<FMediaIOConnection> OtherSources = GetConnections();

	FMediaIOOutputConfiguration DefaultOutputConfiguration = GetDefaultOutputConfiguration();
	Results.Reset(OutputConfigurations.Num() * 4);

	int32 LastDeviceIndex = INDEX_NONE;
	bool bCanDoKeyAndFill = false;

	for (const FMediaIOConfiguration& OutputConfiguration : OutputConfigurations)
	{
		auto BuildList = [&]()
		{
			DefaultOutputConfiguration.MediaConfiguration = OutputConfiguration;

			DefaultOutputConfiguration.OutputReference = EMediaIOReferenceType::FreeRun;
			Results.Add(DefaultOutputConfiguration);

			DefaultOutputConfiguration.OutputReference = EMediaIOReferenceType::External;
			Results.Add(DefaultOutputConfiguration);

			// Add all inputs for reference input
			DefaultOutputConfiguration.OutputReference = EMediaIOReferenceType::Input;
			for (const FMediaIOConnection& InputPort : OtherSources)
			{
				if (InputPort.Device == OutputConfiguration.MediaConnection.Device && InputPort.TransportType == OutputConfiguration.MediaConnection.TransportType && InputPort.PortIdentifier != OutputConfiguration.MediaConnection.PortIdentifier)
				{
					if (InputPort.TransportType != EMediaIOTransportType::QuadLink || InputPort.QuadTransportType == OutputConfiguration.MediaConnection.QuadTransportType)
					{
						if (DefaultOutputConfiguration.OutputType != EMediaIOOutputType::FillAndKey || !(InputPort.PortIdentifier == DefaultOutputConfiguration.KeyPortIdentifier))
						{
							DefaultOutputConfiguration.ReferencePortIdentifier = InputPort.PortIdentifier;
							Results.Add(DefaultOutputConfiguration);
						}
					}
				}
			}
		};

		// Update the Device Info
		if (OutputConfiguration.MediaConnection.Device.DeviceIdentifier != LastDeviceIndex)
		{
			LastDeviceIndex = OutputConfiguration.MediaConnection.Device.DeviceIdentifier;
			bCanDoKeyAndFill = CanDeviceDoAlpha(OutputConfiguration.MediaConnection.Device);
		}

		// Build the list for fill only
		DefaultOutputConfiguration.OutputType = EMediaIOOutputType::Fill;
		BuildList();

		// Add all output port for key
		if (bCanDoKeyAndFill)
		{
			DefaultOutputConfiguration.OutputType = EMediaIOOutputType::FillAndKey;
			for (const FMediaIOConnection& OutputPort : OtherSources)
			{
				if (OutputPort.Device == OutputConfiguration.MediaConnection.Device && OutputPort.TransportType == OutputConfiguration.MediaConnection.TransportType && OutputPort.PortIdentifier != OutputConfiguration.MediaConnection.PortIdentifier)
				{
					if (OutputPort.TransportType != EMediaIOTransportType::QuadLink || OutputPort.QuadTransportType == OutputConfiguration.MediaConnection.QuadTransportType)
					{
						DefaultOutputConfiguration.KeyPortIdentifier = OutputPort.PortIdentifier;
						BuildList();
					}
				}
			}
		}
	}

	return Results;
}

TArray<FMediaIODevice> FAjaDeviceProvider::GetDevices() const
{
	TArray<FMediaIODevice> Results;
	if (!FAja::IsInitialized() || !FAja::CanUseAJACard())
	{
		return Results;
	}

	AJA::AJADeviceScanner DeviceScanner;
	int32 NumDevices = DeviceScanner.GetNumDevices();
	for (int32 DeviceIndex = 0; DeviceIndex < NumDevices; ++DeviceIndex)
	{
		AJA::AJADeviceScanner::DeviceInfo DeviceInfo;
		if (!DeviceScanner.GetDeviceInfo(DeviceIndex, DeviceInfo))
		{
			continue;
		}

		if (!DeviceInfo.bIsSupported)
		{
			continue;
		}

		TCHAR DeviceNameBuffer[AJA::AJADeviceScanner::FormatedTextSize];
		if (!DeviceScanner.GetDeviceTextId(DeviceIndex, DeviceNameBuffer))
		{
			continue;
		}

		FMediaIODevice Device;
		Device.DeviceIdentifier = DeviceIndex;
		Device.DeviceName = DeviceNameBuffer;
		Results.Add(Device);
	}

	return Results;
}

TArray<FMediaIOMode> FAjaDeviceProvider::GetModes(const FMediaIODevice& InDevice, bool bInOutput) const
{
	TArray<FMediaIOMode> Results;
	if (!FAja::IsInitialized() || !InDevice.IsValid() || !FAja::CanUseAJACard())
	{
		return Results;
	}

	AJA::AJADeviceScanner DeviceScanner;
	AJA::AJADeviceScanner::DeviceInfo DeviceInfo;
	if (!DeviceScanner.GetDeviceInfo(InDevice.DeviceIdentifier, DeviceInfo))
	{
		return Results;
	}

	if (!DeviceInfo.bIsSupported)
	{
		return Results;
	}

	const bool bDeviceHasInput = DeviceInfo.NumSdiInput > 0 || DeviceInfo.NumHdmiInput > 0;
	if (!bInOutput && !bDeviceHasInput)
	{
		return Results;
	}

	const bool bDeviceHasOutput = DeviceInfo.NumSdiOutput > 0; // || DeviceInfo.NumHdmiOutput > 0 we do not support HDMI output, you should use a normal graphic card.
	if (bInOutput && !bDeviceHasOutput)
	{
		return Results;
	}

	AJA::AJAVideoFormats FrameFormats(InDevice.DeviceIdentifier);
	const int32 NumSupportedFormat = FrameFormats.GetNumSupportedFormat();
	Results.Reserve(NumSupportedFormat);
	for (int32 Index = 0; Index < NumSupportedFormat; ++Index)
	{
		AJA::AJAVideoFormats::VideoFormatDescriptor Descriptor = FrameFormats.GetSupportedFormat(Index);
		if (!AjaDeviceProvider::IsVideoFormatValid(Descriptor))
		{
			continue;
		}
		Results.Add(AjaDeviceProvider::ToMediaMode(Descriptor));
	}

	return Results;
}

TArray<FMediaIOVideoTimecodeConfiguration> FAjaDeviceProvider::GetTimecodeConfigurations() const
{
	TArray<FMediaIOVideoTimecodeConfiguration> MediaConfigurations;
	bool bHasInputConfiguration = false;
	{
		TArray<FMediaIOConfiguration> InputConfigurations = GetConfigurations(true, false);

		FMediaIOVideoTimecodeConfiguration DefaultTimecodeConfiguration;
		MediaConfigurations.Reset(InputConfigurations.Num() * 2);
		for (const FMediaIOConfiguration& InputConfiguration : InputConfigurations)
		{
			DefaultTimecodeConfiguration.MediaConfiguration = InputConfiguration;
			DefaultTimecodeConfiguration.TimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::LTC;
			MediaConfigurations.Add(DefaultTimecodeConfiguration);

			DefaultTimecodeConfiguration.TimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::VITC;
			MediaConfigurations.Add(DefaultTimecodeConfiguration);
		}
	}
	return MediaConfigurations;
}

TArray<FAjaMediaTimecodeConfiguration> FAjaDeviceProvider::GetTimecodeConfiguration() const
{
	TArray<FAjaMediaTimecodeConfiguration> MediaConfigurations;
	bool bHasInputConfiguration = false;
	{
		TArray<FMediaIOConfiguration> InputConfigurations = GetConfigurations(true, false);

		FAjaMediaTimecodeConfiguration DefaultTimecodeConfiguration;
		MediaConfigurations.Reset(InputConfigurations.Num() * 2);
		for (const FMediaIOConfiguration& InputConfiguration : InputConfigurations)
		{
			DefaultTimecodeConfiguration.MediaConfiguration = InputConfiguration;
			DefaultTimecodeConfiguration.TimecodeFormat = EMediaIOTimecodeFormat::LTC;
			MediaConfigurations.Add(DefaultTimecodeConfiguration);

			DefaultTimecodeConfiguration.TimecodeFormat = EMediaIOTimecodeFormat::VITC;
			MediaConfigurations.Add(DefaultTimecodeConfiguration);
		}
	}
	return MediaConfigurations;
}

TArray<FAjaMediaTimecodeReference> FAjaDeviceProvider::GetTimecodeReferences() const
{
	TArray<FAjaMediaTimecodeReference> Results;
	if (!FAja::IsInitialized() || !FAja::CanUseAJACard())
	{
		return Results;
	}

	const TArrayView<const FCommonFrameRateInfo> AllFrameRates = FCommonFrameRates::GetAll();

	FAjaMediaTimecodeReference DefaultFAjaMediaTimecodeReference = FAjaMediaTimecodeReference();

	AJA::AJADeviceScanner DeviceScanner;
	int32 NumDevices = DeviceScanner.GetNumDevices();
	for (int32 DeviceIndex = 0; DeviceIndex < NumDevices; ++DeviceIndex)
	{
		AJA::AJADeviceScanner::DeviceInfo DeviceInfo;
		if (!DeviceScanner.GetDeviceInfo(DeviceIndex, DeviceInfo))
		{
			continue;
		}

		if (!DeviceInfo.bIsSupported)
		{
			continue;
		}

		if (DeviceInfo.NumberOfLtcInput > 0)
		{
			TCHAR DeviceNameBuffer[AJA::AJADeviceScanner::FormatedTextSize];
			if (!DeviceScanner.GetDeviceTextId(DeviceIndex, DeviceNameBuffer))
			{
				continue;
			}

			DefaultFAjaMediaTimecodeReference.Device.DeviceIdentifier = DeviceIndex;
			DefaultFAjaMediaTimecodeReference.Device.DeviceName = DeviceNameBuffer;

			for (uint32 LtcIndex = 0; LtcIndex < DeviceInfo.NumberOfLtcInput; ++LtcIndex)
			{
				DefaultFAjaMediaTimecodeReference.LtcIndex = LtcIndex + 1;


				for (const FCommonFrameRateInfo& FrameRate : AllFrameRates)
				{
					if (FrameRate.FrameRate.AsDecimal() <= (30.0f + SMALL_NUMBER))
					{
						DefaultFAjaMediaTimecodeReference.LtcFrameRate = FrameRate.FrameRate;
						Results.Add(DefaultFAjaMediaTimecodeReference);
					}
				}
			}
		}
	}

	return Results;
}

FMediaIOConfiguration FAjaDeviceProvider::GetDefaultConfiguration() const
{
	FMediaIOConfiguration Result;
	Result.bIsInput = true;
	Result.MediaConnection.Device.DeviceIdentifier = 0;
	Result.MediaConnection.Protocol = GetProtocolName();
	Result.MediaConnection.PortIdentifier = 1;
	Result.MediaConnection.TransportType = EMediaIOTransportType::SingleLink;
	Result.MediaMode = GetDefaultMode();
	return Result;
}

FMediaIOMode FAjaDeviceProvider::GetDefaultMode() const
{
	FMediaIOMode Mode;
	Mode.DeviceModeIdentifier = AjaMediaOption::DefaultVideoFormat;
	Mode.FrameRate = FFrameRate(30, 1);
	Mode.Resolution = FIntPoint(1920, 1080);
	Mode.Standard = EMediaIOStandardType::Progressive;
	return Mode;
}

FMediaIOInputConfiguration FAjaDeviceProvider::GetDefaultInputConfiguration() const
{
	FMediaIOInputConfiguration Configuration;
	Configuration.MediaConfiguration = GetDefaultConfiguration();
	Configuration.MediaConfiguration.bIsInput = true;
	Configuration.InputType = EMediaIOInputType::Fill;
	return Configuration;
}

FMediaIOOutputConfiguration FAjaDeviceProvider::GetDefaultOutputConfiguration() const
{
	FMediaIOOutputConfiguration Configuration;
	Configuration.MediaConfiguration = GetDefaultConfiguration();
	Configuration.MediaConfiguration.bIsInput = false;
	Configuration.OutputReference = EMediaIOReferenceType::FreeRun;
	Configuration.OutputType = EMediaIOOutputType::Fill;
	return Configuration;
}

FMediaIOVideoTimecodeConfiguration FAjaDeviceProvider::GetDefaultTimecodeConfiguration() const
{
	FMediaIOVideoTimecodeConfiguration Configuration;
	Configuration.MediaConfiguration = GetDefaultConfiguration();
	return Configuration;
}

#undef LOCTEXT_NAMESPACE
