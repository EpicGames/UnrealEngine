// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaMediaDefinitions.h"

#include "AJALib.h"

namespace UE::AjaMedia
{
	FAjaMediaHDROptions MakeAjaMediaHDROptions(const AJA::FAjaHDROptions& HDROptions)
	{
		FAjaMediaHDROptions HDRMetadata;

		HDRMetadata.Gamut = (EAjaHDRMetadataGamut)HDROptions.Gamut;
		HDRMetadata.EOTF = (EAjaHDRMetadataEOTF)HDROptions.EOTF;

		return HDRMetadata;
	}

	AJA::FAjaHDROptions MakeAjaHDROptions(const FAjaMediaHDROptions& HDROptions)
	{
		AJA::FAjaHDROptions HDRMetadata;

		HDRMetadata.Gamut = (AJA::EAjaHDRMetadataGamut)HDROptions.Gamut;
		HDRMetadata.EOTF = (AJA::EAjaHDRMetadataEOTF)HDROptions.EOTF;

		return HDRMetadata;
	}
}
