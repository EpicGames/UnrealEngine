// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAjaMediaModule.h"

#include "Aja/Aja.h"
#include "AjaDeviceProvider.h"
#include "AJALib.h"
#include "Brushes/SlateImageBrush.h"
#include "CoreMinimal.h"
#include "IMediaIOCoreModule.h"
#include "Interfaces/IPluginManager.h"
#include "Modules/ModuleManager.h"
#include "Player/AjaMediaPlayer.h"
#include "Styling/SlateStyle.h"


DEFINE_LOG_CATEGORY(LogAjaMedia);

#define LOCTEXT_NAMESPACE "AjaMediaModule"

/**
 * Implements the AJAMedia module.
 */
class FAjaMediaModule : public IAjaMediaModule
{
public:

	//~ IAjaMediaModule interface
	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		return MakeShared<FAjaMediaPlayer, ESPMode::ThreadSafe>(EventSink);
	}

	virtual TSharedPtr<FSlateStyleSet> GetStyle() override
	{
		return StyleSet;
	}

	virtual bool IsInitialized() const override { return true; }

	virtual bool CanBeUsed() const override { return FAja::CanUseAJACard(); }

public:

	//~ IModuleInterface interface
	virtual void StartupModule() override
	{
		CreateStyle();

		IMediaIOCoreModule::Get().RegisterDeviceProvider(&DeviceProvider);
	}

	virtual void ShutdownModule() override
	{
		if (IMediaIOCoreModule::IsAvailable())
		{
			IMediaIOCoreModule::Get().UnregisterDeviceProvider(&DeviceProvider);
		}
	}

private:
	void CreateStyle()
	{
		static FName StyleName(TEXT("AjaMediaStyle"));
		StyleSet = MakeShared<FSlateStyleSet>(StyleName);

		const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("AjaMedia"))->GetContentDir();
		const FVector2D Icon16x16(16.0f, 16.0f);
		StyleSet->Set("AjaMediaIcon", new FSlateImageBrush((ContentDir / TEXT("Editor/Icons/AjaMediaSource_64x")) + TEXT(".png"), Icon16x16));
	}

private:
	FAjaDeviceProvider DeviceProvider;
	TSharedPtr<FSlateStyleSet> StyleSet;
};

IMPLEMENT_MODULE(FAjaMediaModule, AjaMedia);

#undef LOCTEXT_NAMESPACE
