// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include <windows.h>

#include "AJALib.h"

#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include "CoreMinimal.h"
#include "AjaMediaSettings.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAjaMedia, Log, All);

namespace AjaMediaOption
{
	static const FName DeviceIndex("DeviceIndex");
	static const FName PortIndex("PortIndex");
	static const FName TransportType("TransportType");
	static const FName QuadTransportType("QuadTransportType");
	static const FName TimecodeFormat("TimecodeFormat");
	static const FName LogDropFrame("LogDropFrame");
	static const FName EncodeTimecodeInTexel("EncodeTimecodeInTexel");
	static const FName CaptureWithAutoCirculating("CaptureWithAutoCirculating");
	static const FName StopOnTimeout("StopOnTimeout");
	static const FName CaptureAncillary("CaptureAncillary");
	static const FName CaptureAudio("CaptureAudio");
	static const FName CaptureVideo("CaptureVideo");
	static const FName MaxAncillaryFrameBuffer("MaxAncillaryFrameBuffer");
	static const FName AudioChannel("AudioChannel");
	static const FName MaxAudioFrameBuffer("MaxAudioFrameBuffer");
	static const FName AjaVideoFormat("AjaVideoFormat");
	static const FName ColorFormat("ColorFormat");
	static const FName SRGBInput("sRGBInput");
	static const FName MaxVideoFrameBuffer("MaxVideoFrameBuffer");

	static const AJA::FAJAVideoFormat DefaultVideoFormat = 9; // 1080p3000
}
