// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaCustomTimeStep.h"
#include "AjaMediaPrivate.h"
#include "Aja/Aja.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "Misc/App.h"

//~ IAJASyncChannelCallbackInterface implementation
//--------------------------------------------------------------------
// Those are called from the AJA thread. There's a lock inside AJA to prevent this object from dying while in this thread.
struct UAjaCustomTimeStep::FAJACallback : public AJA::IAJASyncChannelCallbackInterface
{
	UAjaCustomTimeStep* Owner;
	FAJACallback(UAjaCustomTimeStep* InOwner)
		: Owner(InOwner)
	{}

	//~ IAJAInputCallbackInterface interface
	virtual void OnInitializationCompleted(bool bSucceed) override
	{
		Owner->State = bSucceed ? ECustomTimeStepSynchronizationState::Synchronizing : ECustomTimeStepSynchronizationState::Error;
		if (!bSucceed)
		{
			UE_LOGF(LogAjaMedia, Error, "The initialization of '%ls' failed. The CustomTimeStep won't be synchronized.", *Owner->GetName());
		}
	}
};


//~ UFixedFrameRateCustomTimeStep implementation
//--------------------------------------------------------------------
UAjaCustomTimeStep::UAjaCustomTimeStep(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bUseReferenceIn(false)
	, TimecodeFormat(EMediaIOTimecodeFormat::LTC)
	, bEnableOverrunDetection(true)
	, SyncChannel(nullptr)
	, SyncCallback(nullptr)
#if WITH_EDITORONLY_DATA
	, InitializedEngine(nullptr)
	, LastAutoSynchronizeInEditorAppTime(0.0)
	, LastAutoDetectInEditorAppTime(0.0)
#endif
	, State(ECustomTimeStepSynchronizationState::Closed)
	, bWarnedAboutVSync(false)
	, bIsPreviousSyncCountValid(false)
	, PreviousSyncCount(0)
	, SyncCountDelta(0)
	, LastDetectedVideoFormat(0) // 0 means UNKNOWN
	, bLastDetectedVideoFormatInitialized(false)
{
	MediaConfiguration.bIsInput = !bUseReferenceIn;
	DeviceProvider = MakeUnique<FAjaDeviceProvider>();
}

bool UAjaCustomTimeStep::Initialize(UEngine* InEngine)
{
#if WITH_EDITORONLY_DATA
	InitializedEngine = nullptr;
#endif

	State = ECustomTimeStepSynchronizationState::Closed;

	bLastDetectedVideoFormatInitialized = false;

	if (!FAja::IsInitialized())
	{
		State = ECustomTimeStepSynchronizationState::Error;
		UE_LOGF(LogAjaMedia, Error, "The CustomTimeStep '%ls' can't be initialized. AJA is not initialized on your machine.", *GetName());
		return false;
	}

	if (!FAja::CanUseAJACard())
	{
		State = ECustomTimeStepSynchronizationState::Error;
		UE_LOGF(LogAjaMedia, Warning, "The CustomTimeStep '%ls' can't be initialized because Aja card cannot be used. Are you in a Commandlet? You may override this behavior by launching with -ForceAjaUsage", *GetName());
		return false;
	}

	if (bUseReferenceIn && bWaitForFrameToBeReady)
	{
		UE_LOGF(LogAjaMedia, Warning, "The CustomTimeStep '%ls' use both the reference and wait for the frame to be ready. These options are not compatible.", *GetName());
	}

	if (bWaitForFrameToBeReady && MediaConfiguration.MediaMode.Standard == EMediaIOStandardType::Interlaced)
	{
		UE_LOGF(LogAjaMedia, Warning, "The CustomTimeStep '%ls' is waiting for the frame to be ready and interlaced picture is not supported.", *GetName());
	}

	if (bAutoDetectFormat && !bUseReferenceIn)
	{
		constexpr bool bReinitialize = true;
		DetectConfiguration(InEngine, bReinitialize);
		return true;
	}
	else
	{
		FString FailureReason;
        if (!MediaConfiguration.IsValid())
        {
        	State = ECustomTimeStepSynchronizationState::Error;
        	UE_LOGF(LogAjaMedia, Error, "The CustomTimeStep '%ls' configuration is invalid.", *GetName());
        	return false;
        }
	}

	return Initialize_Internal(InEngine);
}

void UAjaCustomTimeStep::Shutdown(UEngine* InEngine)
{
#if WITH_EDITORONLY_DATA
	InitializedEngine = nullptr;
#endif

	State = ECustomTimeStepSynchronizationState::Closed;
	ReleaseResources();
}

bool UAjaCustomTimeStep::VerifyGenlockSignal()
{
	bool bGenlockSignalValid = true;

	AJA::FAJAVideoFormat VideoFormat;
	const bool bHasGenlockSignal = SyncChannel->GetVideoFormat(VideoFormat);

	// Log error only once per signal change
	const bool bShouldLogError = !bAutoDetectFormat && (!bLastDetectedVideoFormatInitialized || (LastDetectedVideoFormat != VideoFormat));

	if (bHasGenlockSignal)
	{
		if (VideoFormat != MediaConfiguration.MediaMode.DeviceModeIdentifier)
		{
			bGenlockSignalValid = false;

			if (bShouldLogError)
			{
				constexpr uint32 MaxStringLen = 128;
				char VideoFormatStdString[MaxStringLen];
				char ExpectedVideoFormatStdString[MaxStringLen];

				const bool bVideoFormatStringOk = AJA::AJAVideoFormats::VideoFormatToString(
					VideoFormat, VideoFormatStdString, sizeof(VideoFormatStdString)
				);

				const bool bExpectedVideoFormatStringOk = AJA::AJAVideoFormats::VideoFormatToString(
					MediaConfiguration.MediaMode.DeviceModeIdentifier, ExpectedVideoFormatStdString, sizeof(ExpectedVideoFormatStdString)
				);

				if (bVideoFormatStringOk && bExpectedVideoFormatStringOk)
				{
					FString VideoFormatString(UTF8_TO_TCHAR(VideoFormatStdString));
					FString ExpectedVideoFormatString(UTF8_TO_TCHAR(ExpectedVideoFormatStdString));

					UE_LOGF(LogAjaMedia, Warning, "Detected Genlock signal '%ls' differs from expected '%ls'", *VideoFormatString, *ExpectedVideoFormatString);
				}
				else
				{
					UE_LOGF(LogAjaMedia, Warning, "Detected Genlock signal '%d' differs from expected '%d'", 
						VideoFormat, MediaConfiguration.MediaMode.DeviceModeIdentifier
					);
				}
			}
		}
	}
	else
	{
		bGenlockSignalValid = false;

		if (bShouldLogError)
		{
			UE_LOGF(LogAjaMedia, Warning, "Genlock signal not detected");
		}
	}

	LastDetectedVideoFormat = VideoFormat;
	bLastDetectedVideoFormatInitialized = true;
	return bGenlockSignalValid;
}

bool UAjaCustomTimeStep::Initialize_Internal(UEngine* InEngine)
{
	check(SyncCallback == nullptr);
	SyncCallback = new FAJACallback(this);

	AJA::AJADeviceOptions DeviceOptions(MediaConfiguration.MediaConnection.Device.DeviceIdentifier);

	//Convert Port Index to match what AJA expects
	AJA::AJASyncChannelOptions Options(*GetName());
	Options.CallbackInterface = SyncCallback;
	Options.ChannelIndex = MediaConfiguration.MediaConnection.PortIdentifier;
	Options.VideoFormatIndex = MediaConfiguration.MediaMode.DeviceModeIdentifier;
	Options.bOutput = bUseReferenceIn;
	Options.bWaitForFrameToBeReady = bWaitForFrameToBeReady && !bUseReferenceIn;
	Options.TransportType = AJA::ETransportType::TT_SdiSingle;
	Options.bAutoDetectFormat = bAutoDetectFormat;
	Options.bStopOnCardTimeout = bStopOnCardTimeout;

	{
		const EMediaIOTransportType TransportType = MediaConfiguration.MediaConnection.TransportType;
		const EMediaIOQuadLinkTransportType QuadTransportType = MediaConfiguration.MediaConnection.QuadTransportType;
		switch (TransportType)
		{
		case EMediaIOTransportType::SingleLink:
			Options.TransportType = AJA::ETransportType::TT_SdiSingle;
			break;
		case EMediaIOTransportType::DualLink:
			Options.TransportType = AJA::ETransportType::TT_SdiDual;
			break;
		case EMediaIOTransportType::QuadLink:
			Options.TransportType = QuadTransportType == EMediaIOQuadLinkTransportType::SquareDivision ? AJA::ETransportType::TT_SdiQuadSQ : AJA::ETransportType::TT_SdiQuadTSI;
			break;
		case EMediaIOTransportType::HDMI:
			Options.TransportType = AJA::ETransportType::TT_Hdmi;
			break;
		}
	}

	Options.TimecodeFormat = AJA::ETimecodeFormat::TCF_None;
	if (!Options.bOutput)
	{
		switch (TimecodeFormat)
		{
		case EMediaIOTimecodeFormat::None:
			Options.TimecodeFormat = AJA::ETimecodeFormat::TCF_None;
			break;
		case EMediaIOTimecodeFormat::LTC:
			Options.TimecodeFormat = AJA::ETimecodeFormat::TCF_LTC;
			break;
		case EMediaIOTimecodeFormat::VITC:
			Options.TimecodeFormat = AJA::ETimecodeFormat::TCF_VITC1;
			break;
		default:
			break;
		}
	}

	check(SyncChannel == nullptr);
	SyncChannel = new AJA::AJASyncChannel();
	if (!SyncChannel->Initialize(DeviceOptions, Options))
	{
		State = ECustomTimeStepSynchronizationState::Error;
		delete SyncChannel;
		SyncChannel = nullptr;
		delete SyncCallback;
		SyncCallback = nullptr;
		return false;
	}

#if WITH_EDITORONLY_DATA
	InitializedEngine = InEngine;
#endif

	return true;
}

void UAjaCustomTimeStep::OnConfigurationAutoDetected(TArray<FAjaDeviceProvider::FMediaIOConfigurationWithTimecodeFormat> InConfigurations, UEngine* InEngine, bool bReinitialize)
{
	bool bConfigurationFound = false;
	for (const FAjaDeviceProvider::FMediaIOConfigurationWithTimecodeFormat& Configuration : InConfigurations)
	{
		if (Configuration.Configuration.MediaConnection.Device.DeviceIdentifier == MediaConfiguration.MediaConnection.Device.DeviceIdentifier
			&& Configuration.Configuration.MediaConnection.PortIdentifier == MediaConfiguration.MediaConnection.PortIdentifier)
		{
			MediaConfiguration = Configuration.Configuration;
			bConfigurationFound = true;
			break;
		}
	}
	if (!bConfigurationFound)
	{
		UE_LOGF(LogAjaMedia, Warning, "No configuration was detected for Custom Time Step '%ls'", *GetName());
		ReleaseResources();
		return;
	}
	
	if (bReinitialize)
	{
		Initialize_Internal(InEngine);
	}
}

bool UAjaCustomTimeStep::UpdateTimeStep(UEngine* InEngine)
{
	if (State == ECustomTimeStepSynchronizationState::Closed)
	{
		return true;
	}

	if (State == ECustomTimeStepSynchronizationState::Error)
	{
		ReleaseResources();
		bLastDetectedVideoFormatInitialized = false;
		bIsPreviousSyncCountValid = false;
 
		// In Editor only, when not in pie, reinitialized the device
#if WITH_EDITORONLY_DATA && WITH_EDITOR
		if (InitializedEngine && !GIsPlayInEditorWorld && GIsEditor)
		{
			constexpr double TimeBetweenAttempt = 1.0;
			if (FApp::GetCurrentTime() - LastAutoSynchronizeInEditorAppTime > TimeBetweenAttempt)
			{
				Initialize(InitializedEngine);
				LastAutoSynchronizeInEditorAppTime = FApp::GetCurrentTime();
			}
		}
#endif
		return true;
	}

	check(State == ECustomTimeStepSynchronizationState::Synchronized || State == ECustomTimeStepSynchronizationState::Synchronizing);

	// Warn about Vsync once
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VSync"));
		if (!bWarnedAboutVSync)
		{
			bool bLockToVsync = CVar->GetValueOnGameThread() != 0;
			if (bLockToVsync)
			{
				UE_LOGF(LogAjaMedia, Warning, "The Engine is using VSync and may break the 'genlock'");
				bWarnedAboutVSync = true;
			}
		}
	}

	// Updates logical last time to match logical current time from last tick
	UpdateApplicationLastTime();

	const double TimeBeforeSync = FPlatformTime::Seconds();

	const bool bValidGenlockSignal = VerifyGenlockSignal();
	const bool bWaitedForSync = WaitForSync();

	const double TimeAfterSync = FPlatformTime::Seconds();

#if WITH_EDITORONLY_DATA && WITH_EDITOR
	if (!bValidGenlockSignal && bAutoDetectFormat)
	{
		if (InitializedEngine && !GIsPlayInEditorWorld && GIsEditor)
		{
			constexpr double TimeBetweenAttempt = 3.0;
			if (FApp::GetCurrentTime() - LastAutoDetectInEditorAppTime > TimeBetweenAttempt)
			{
				constexpr bool bReinitialize = false;
				DetectConfiguration(InEngine, bReinitialize);
				LastAutoDetectInEditorAppTime = FApp::GetCurrentTime();
			}
		}
	}
#endif

	if (!bWaitedForSync)
	{
		if (bAutoDetectFormat)
		{
			State = ECustomTimeStepSynchronizationState::Closed;
			return true;
		}
		else
		{
			State = ECustomTimeStepSynchronizationState::Error;
			return true;
		}
	}

	if (bValidGenlockSignal)
	{
		State = ECustomTimeStepSynchronizationState::Synchronized;
	}
	else
	{
		State = ECustomTimeStepSynchronizationState::Synchronizing;
	}

	UpdateAppTimes(TimeBeforeSync, TimeAfterSync);

	return false;
}

ECustomTimeStepSynchronizationState UAjaCustomTimeStep::GetSynchronizationState() const
{
	return State;
}

FFrameRate UAjaCustomTimeStep::GetFixedFrameRate() const
{
	return MediaConfiguration.MediaMode.FrameRate;
}

FFrameRate UAjaCustomTimeStep::GetSyncRate() const
{
	FFrameRate SyncRate = GetFixedFrameRate();

	if (MediaConfiguration.MediaMode.Standard == EMediaIOStandardType::ProgressiveSegmentedFrame || bWaitForBothFields)
	{
		// If pSF you should get 2 field interrupts.
		SyncRate.Numerator *= 2;
	}

	return SyncRate;
}

uint32 UAjaCustomTimeStep::GetLastSyncCountDelta() const
{
	return SyncCountDelta;
}

bool UAjaCustomTimeStep::IsLastSyncDataValid() const
{
	return bIsPreviousSyncCountValid;
}

//~ UObject implementation
//--------------------------------------------------------------------
void UAjaCustomTimeStep::BeginDestroy()
{
	ReleaseResources();
	Super::BeginDestroy();
}

#if WITH_EDITOR
void UAjaCustomTimeStep::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAjaCustomTimeStep, bUseReferenceIn))
	{
		MediaConfiguration = FMediaIOConfiguration();
		MediaConfiguration.bIsInput = !bUseReferenceIn;
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

//~ UAjaCustomTimeStep implementation
//--------------------------------------------------------------------
bool UAjaCustomTimeStep::WaitForSync()
{
	check(SyncChannel);

	SyncCountDelta = 1;

	const bool bWaitIsValid = SyncChannel->WaitForSync();

	if (!bWaitIsValid)
	{
#if WITH_EDITORONLY_DATA
		if (bAutoDetectFormat && InitializedEngine)
		{
			UE_LOGF(LogAjaMedia, Display, "Custom time step sync format changed, restarting time step...");
			bIgnoreWarningForOneFrame = true;
			UEngine* Engine = InitializedEngine;
			Shutdown(Engine);
			Initialize(Engine);
			return false;
		}
#endif

		State = ECustomTimeStepSynchronizationState::Error;
		bIsPreviousSyncCountValid = false;
		if (!bIgnoreWarningForOneFrame)
		{
			UE_LOGF(LogAjaMedia, Error, "The Engine couldn't run fast enough to keep up with the CustomTimeStep Sync. The wait timed out.");
		}

		return false;
	}

	uint32 NewSyncCount = 0;
	const bool bIsNewSyncCountValid = SyncChannel->GetSyncCount(NewSyncCount);

	const int32 ExpectedSyncCountsPerWait = GetExpectedSyncCountDelta();

	if (bIsNewSyncCountValid 
		&& bIsPreviousSyncCountValid 
		&& (NewSyncCount != (PreviousSyncCount + ExpectedSyncCountsPerWait)))
	{
		uint32 Difference = NewSyncCount - PreviousSyncCount - ExpectedSyncCountsPerWait;
		if (IsInterlaced() && UGenlockedCustomTimeStep::CVarExperimentalFieldFlipFix.GetValueOnAnyThread() && Difference % 2 != 0)
		{
			// Try not dropping an odd number of frames.
			bool bSyncSuccess = true;
			while (Difference % 2 != 0 && bSyncSuccess)
			{
				bSyncSuccess = SyncChannel->WaitForSync();
				if (bSyncSuccess)
				{
					SyncChannel->GetSyncCount(NewSyncCount);
					Difference = NewSyncCount - PreviousSyncCount - ExpectedSyncCountsPerWait;
				}
				else
				{
					UE_LOGF(LogAjaMedia, Error, "The Engine couldn't run fast enough to keep up with the CustomTimeStep Sync. The wait timed out.");
				}
			}
		}

		if (bEnableOverrunDetection)
		{
			UE_LOGF(LogAjaMedia, Warning, 
				"The Engine couldn't run fast enough to keep up with the CustomTimeStep Sync. '%d' frame(s) dropped.", 
				NewSyncCount - PreviousSyncCount - ExpectedSyncCountsPerWait);
		}
	}

	if (bIsNewSyncCountValid)
	{
		if (bIsPreviousSyncCountValid)
		{
			SyncCountDelta = NewSyncCount - PreviousSyncCount;
		}

		PreviousSyncCount = NewSyncCount;
	}

	bIsPreviousSyncCountValid = bIsNewSyncCountValid;
	if (bIsPreviousSyncCountValid)
	{
		bEncounteredInvalidAutoDetectFrame = false;
	}

	return true;
}

void UAjaCustomTimeStep::ReleaseResources()
{
	if (SyncChannel)
	{
		SyncChannel->Uninitialize();
		delete SyncChannel;
		SyncChannel = nullptr;
		delete SyncCallback;
		SyncCallback = nullptr;
	}

	bWarnedAboutVSync = false;
	bIsPreviousSyncCountValid = false;
	
	if (DeviceProvider)
	{
		DeviceProvider->EndAutoDetectConfiguration();
	}
}

void UAjaCustomTimeStep::DetectConfiguration(class UEngine* InEngine, bool bReinitialize)
{
	if (DeviceProvider)
	{
		DeviceProvider->AutoDetectConfiguration(FAjaDeviceProvider::FOnConfigurationAutoDetected::CreateUObject(this, &UAjaCustomTimeStep::OnConfigurationAutoDetected, InEngine, bReinitialize));
	}
}

bool UAjaCustomTimeStep::IsInterlaced() const
{
	return MediaConfiguration.MediaMode.Standard == EMediaIOStandardType::Interlaced;
}

