// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaMediaSettings.h"

/* UAjaMediaSettings
*****************************************************************************/
