// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaMediaSource.h"

#include "Aja/Aja.h"
#include "AjaDeviceProvider.h"
#include "AjaMediaPrivate.h"
#include "IMediaIOCoreModule.h"
#include "MediaIOCorePlayerBase.h"

UAjaMediaSource::UAjaMediaSource()
	: UCaptureCardMediaSource()
	, bCaptureWithAutoCirculating(true)
	, bCaptureAncillary(false)
	, MaxNumAncillaryFrameBuffer(8)
	, bCaptureAudio(false)
	, AudioChannel(EAjaMediaAudioChannel::Channel8)
	, MaxNumAudioFrameBuffer(8)
	, bCaptureVideo(true)
	, ColorFormat(EAjaMediaSourceColorFormat::YUV2_8bit)
	, MaxNumVideoFrameBuffer(8)
	, bLogDropFrame(true)
	, bEncodeTimecodeInTexel(false)
{
	AssignDefaultConfiguration();
}

/*
 * IMediaOptions interface
 */

bool UAjaMediaSource::GetMediaOption(const FName& Key, bool DefaultValue) const
{
	if (Key == AjaMediaOption::CaptureWithAutoCirculating)
	{
		return bCaptureWithAutoCirculating;
	}
	if (Key == AjaMediaOption::CaptureAncillary)
	{
		return bCaptureAncillary;
	}
	if (Key == AjaMediaOption::CaptureAudio)
	{
		return bCaptureAudio;
	}
	if (Key == AjaMediaOption::CaptureVideo)
	{
		return bCaptureVideo;
	}
	if (Key == AjaMediaOption::LogDropFrame)
	{
		return bLogDropFrame;
	}
	if (Key == AjaMediaOption::EncodeTimecodeInTexel)
	{
		return bEncodeTimecodeInTexel;
	}
	if (Key == AjaMediaOption::StopOnTimeout)
	{
		return bStopInputOnCardTimeout;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

int64 UAjaMediaSource::GetMediaOption(const FName& Key, int64 DefaultValue) const
{
	if (Key == AjaMediaOption::DeviceIndex)
	{
		return MediaConfiguration.MediaConnection.Device.DeviceIdentifier;
	}
	if (Key == AjaMediaOption::PortIndex)
	{
		return MediaConfiguration.MediaConnection.PortIdentifier;
	}
	if (Key == AjaMediaOption::TransportType)
	{
		return (int64)MediaConfiguration.MediaConnection.TransportType;
	}
	if (Key == AjaMediaOption::QuadTransportType)
	{
		return (int64)MediaConfiguration.MediaConnection.QuadTransportType;
	}
	if (Key == FMediaIOCoreMediaOption::FrameRateNumerator)
	{
		return MediaConfiguration.MediaMode.FrameRate.Numerator;
	}
	if (Key == FMediaIOCoreMediaOption::FrameRateDenominator)
	{
		return MediaConfiguration.MediaMode.FrameRate.Denominator;
	}
	if (Key == FMediaIOCoreMediaOption::ResolutionWidth)
	{
		return MediaConfiguration.MediaMode.Resolution.X;
	}
	if (Key == FMediaIOCoreMediaOption::ResolutionHeight)
	{
		return MediaConfiguration.MediaMode.Resolution.Y;
	}
	if (Key == AjaMediaOption::TimecodeFormat)
	{
		return (int64)AutoDetectableTimecodeFormat;
	}
	if (Key == AjaMediaOption::MaxAncillaryFrameBuffer)
	{
		return MaxNumAncillaryFrameBuffer;
	}
	if (Key == AjaMediaOption::AudioChannel)
	{
		return (int64)AudioChannel;
	}
	if (Key == AjaMediaOption::MaxAudioFrameBuffer)
	{
		return MaxNumAudioFrameBuffer;
	}
	if (Key == AjaMediaOption::AjaVideoFormat)
	{
		return MediaConfiguration.MediaMode.DeviceModeIdentifier;
	}
	if (Key == AjaMediaOption::ColorFormat)
	{
		return (int64)ColorFormat;
	}
	if (Key == AjaMediaOption::MaxVideoFrameBuffer)
	{
		return MaxNumVideoFrameBuffer;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

FString UAjaMediaSource::GetMediaOption(const FName& Key, const FString& DefaultValue) const
{
	if (Key == FMediaIOCoreMediaOption::VideoModeName)
	{
		return MediaConfiguration.MediaMode.GetModeName().ToString();
	}
	return Super::GetMediaOption(Key, DefaultValue);
}

bool UAjaMediaSource::HasMediaOption(const FName& Key) const
{
	if ((Key == AjaMediaOption::DeviceIndex) ||
		(Key == AjaMediaOption::PortIndex) ||
		(Key == AjaMediaOption::TransportType) ||
		(Key == AjaMediaOption::QuadTransportType) ||
		(Key == FMediaIOCoreMediaOption::FrameRateNumerator) ||
		(Key == FMediaIOCoreMediaOption::FrameRateDenominator) ||
		(Key == FMediaIOCoreMediaOption::ResolutionWidth) ||
		(Key == FMediaIOCoreMediaOption::ResolutionHeight) ||
		(Key == FMediaIOCoreMediaOption::VideoModeName) ||
		(Key == AjaMediaOption::TimecodeFormat) ||
		(Key == AjaMediaOption::CaptureWithAutoCirculating) ||
		(Key == AjaMediaOption::CaptureAncillary) ||
		(Key == AjaMediaOption::CaptureAudio) ||
		(Key == AjaMediaOption::CaptureVideo) ||
		(Key == AjaMediaOption::MaxAncillaryFrameBuffer) ||
		(Key == AjaMediaOption::AudioChannel) ||
		(Key == AjaMediaOption::MaxAudioFrameBuffer) ||
		(Key == AjaMediaOption::AjaVideoFormat) ||
		(Key == AjaMediaOption::ColorFormat) ||
		(Key == AjaMediaOption::MaxVideoFrameBuffer) ||
		(Key == AjaMediaOption::LogDropFrame) ||
		(Key == AjaMediaOption::EncodeTimecodeInTexel)
		)
	{
		return true;
	}

	return Super::HasMediaOption(Key);
}

/*
 * UMediaSource interface
 */

FString UAjaMediaSource::GetUrl() const
{
	return MediaConfiguration.MediaConnection.ToUrl();
}

bool UAjaMediaSource::Validate() const
{
	FString FailureReason;
	if (bAutoDetectInput)
	{
		if (!MediaConfiguration.MediaConnection.IsValid())
		{
			UE_LOGF(LogAjaMedia, Warning, "The MediaConfiguration '%ls' is invalid.", *GetName());
			return false;
		}
	}
	else
	{
		if (!MediaConfiguration.IsValid())
		{
			UE_LOGF(LogAjaMedia, Warning, "The MediaConfiguration '%ls' is invalid.", *GetName());
			return false;
		}
	}
	

	if (!FAja::IsInitialized())
	{
		UE_LOGF(LogAjaMedia, Warning, "Can't validate MediaSource '%ls'. the Aja library was not initialized.", *GetName());
		return false;
	}

	if (!FAja::CanUseAJACard())
	{
		UE_LOGF(LogAjaMedia, Warning, "Can't validate MediaSource '%ls' because Aja card cannot be used. Are you in a Commandlet? You may override this behavior by launching with -ForceAjaUsage", *GetName());
		return false;
	}

	TUniquePtr<AJA::AJADeviceScanner> Scanner = MakeUnique<AJA::AJADeviceScanner>();
	AJA::AJADeviceScanner::DeviceInfo DeviceInfo;
	FMediaIODevice Device = MediaConfiguration.MediaConnection.Device;

	if (!Scanner->GetDeviceInfo(Device.DeviceIdentifier, DeviceInfo))
	{
		UE_LOGF(LogAjaMedia, Warning, "The MediaSource '%ls' use the device '%ls' that doesn't exist on this machine.", *GetName(), *MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	if (!DeviceInfo.bIsSupported)
	{
		UE_LOGF(LogAjaMedia, Warning, "The MediaSource '%ls' use the device '%ls' that is not supported by the AJA SDK.", *GetName(), *MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	if (DeviceInfo.NumSdiInput == 0 && DeviceInfo.NumHdmiInput == 0)
	{
		UE_LOGF(LogAjaMedia, Warning, "The MediaSource '%ls' use the device '%ls' that can't capture.", *GetName(), *MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	if (bCaptureAncillary && !DeviceInfo.bCanDoCustomAnc)
	{
		UE_LOGF(LogAjaMedia, Warning, "The MediaSource '%ls' use the device '%ls' that can't capture Ancillary data.", *GetName(), *MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	if (bUseTimeSynchronization && AutoDetectableTimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::None)
	{
		UE_LOGF(LogAjaMedia, Warning, "The MediaSource '%ls' uses time synchronization but hasn't enabled the timecode.", *GetName());
		return false;
	}

	if (bCaptureVideo)
	{
		if (ColorFormat == EAjaMediaSourceColorFormat::YUV2_8bit && !DeviceInfo.bSupportPixelFormat8bitYCBCR)
		{
			UE_LOGF(LogAjaMedia, Warning, "The MediaSource '%ls' use the device '%ls' that doesn't support the 8bit YUV pixel format.", *GetName(), *MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
			return false;
		}
		if (ColorFormat == EAjaMediaSourceColorFormat::YUV_10bit && !DeviceInfo.bSupportPixelFormat10bitYCBCR)
		{
			UE_LOGF(LogAjaMedia, Warning, "The MediaSource '%ls' use the device '%ls' that doesn't support the 10bit YUV pixel format.", *GetName(), *MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
			return false;
		}
	}

	if (EvaluationType == EMediaIOSampleEvaluationType::Timecode && (!bUseTimeSynchronization || AutoDetectableTimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::None))
	{
		UE_LOGF(LogAjaMedia, Warning, "The MediaSource '%ls' uses 'Timecode' evaluation type which requires time synchronization and timecode enabled.", *GetName());
		return false;
	}

	if (bFramelock && (!bRenderJIT || !bUseTimeSynchronization || AutoDetectableTimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::None))
	{
		UE_LOGF(LogAjaMedia, Warning, "The MediaSource '%ls' uses 'Framelock' which requires JIT rendering, time synchronization and timecode enabled.", *GetName());
		return false;
	}

	if (!bRenderJIT && EvaluationType == EMediaIOSampleEvaluationType::Latest)
	{
		UE_LOGF(LogAjaMedia, Warning, "The MediaSource '%ls' uses 'Latest' evaluation type which requires JIT rendering.", *GetName());
		return false;
	}

	if (bFramelock)
	{
		UE_LOGF(LogAjaMedia, Warning, "The MediaSource '%ls' uses 'Framelock' which has not been implemented yet. This option will be ignored.", *GetName());
	}

	return true;
}

#if WITH_EDITOR
FString UAjaMediaSource::GetDescriptionString() const
{
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(FAjaDeviceProvider::GetProviderName());
	if (DeviceProviderPtr)
	{
		return DeviceProviderPtr->ToText(MediaConfiguration, bAutoDetectInput).ToString(); 
	}

	return Super::GetDescriptionString();
}

void UAjaMediaSource::GetDetailsPanelInfoElements(TArray<TTuple<FText, FText, FText>>& OutInfoElements) const
{
#define LOCTEXT_NAMESPACE "AjaMediaSource"
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(FAjaDeviceProvider::GetProviderName());
	if (!DeviceProviderPtr)
	{
		return;
	}
	
	const FText ConfigHeader = LOCTEXT("MediaConfigurationHeader", "Media Configuration");

	const FText SourceText = FText::Format(LOCTEXT("TransportFormat", "{0} {1}"),
		DeviceProviderPtr->GetTransportName(MediaConfiguration.MediaConnection.TransportType, MediaConfiguration.MediaConnection.QuadTransportType),
		FText::AsNumber(MediaConfiguration.MediaConnection.PortIdentifier));
	
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("DeviceLabel", "Device"), FText::FromName(MediaConfiguration.MediaConnection.Device.DeviceName)));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("SourceLabel", "Source"), SourceText));

	// When auto-detect is on, prefer the live-detected mode published by a running player.
	const bool bShowAuto = bAutoDetectInput;
	const bool bDetectedConfigurationAvailable = bShowAuto && HasLastDetectedConfiguration();
	const FMediaIOConfiguration& EffectiveConfig = bDetectedConfigurationAvailable ? GetLastDetectedConfiguration() : MediaConfiguration;
	const FText AutoNoSignal = LOCTEXT("AutoNoSignal", "Auto (no signal)");

	const FText ResolutionText = (bShowAuto && !bDetectedConfigurationAvailable) ? AutoNoSignal : EffectiveConfig.MediaMode.GetModeName();
	const FText StandardText = (bShowAuto && !bDetectedConfigurationAvailable) ? AutoNoSignal : UEnum::GetDisplayValueAsText(EffectiveConfig.MediaMode.Standard);
	const FText FramerateText = (bShowAuto && !bDetectedConfigurationAvailable) ? AutoNoSignal : EffectiveConfig.MediaMode.FrameRate.ToPrettyText();

	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("ResolutionLabel", "Resolution"), ResolutionText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("StandardLabel", "Standard"), StandardText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("FramerateLabel", "Framerate"), FramerateText));

	const FText AdditionalHeader = LOCTEXT("AdditionalSettingsHeader", "Additional Settings");
	const FText DisabledText = LOCTEXT("Disabled", "Disabled");
	const FText AncillaryBufferText = FText::Format(LOCTEXT("AncillaryBufferFormat", "{0} buffered frames"), FText::AsNumber(MaxNumAncillaryFrameBuffer));
	const FText AudioText = FText::Format(LOCTEXT("AudioFormat", "{0}, {1} buffered frames"),
		UEnum::GetDisplayValueAsText(AudioChannel), FText::AsNumber(MaxNumAudioFrameBuffer));
	
	OutInfoElements.Add(FInfoElement(AdditionalHeader, LOCTEXT("TimecodeFormatLabel", "Timecode Format"), UEnum::GetDisplayValueAsText(AutoDetectableTimecodeFormat)));
	OutInfoElements.Add(FInfoElement(AdditionalHeader, LOCTEXT("AncillaryLabel", "Ancillary"), bCaptureAncillary ? AncillaryBufferText : DisabledText));
	OutInfoElements.Add(FInfoElement(AdditionalHeader, LOCTEXT("AudioLabel", "Audio"), bCaptureAudio ? AudioText : DisabledText));
	OutInfoElements.Add(FInfoElement(AdditionalHeader, LOCTEXT("ColorConversionLabel", "Color Conversion"), FText::FromString(ColorConversionSettings.ToString())));
	
#undef LOCTEXT_NAMESPACE
}
#endif //WITH_EDITOR

#if WITH_EDITOR
bool UAjaMediaSource::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAjaMediaSource, bEncodeTimecodeInTexel))
	{
		return AutoDetectableTimecodeFormat != EMediaIOAutoDetectableTimecodeFormat::None && bCaptureVideo;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UTimeSynchronizableMediaSource, bUseTimeSynchronization))
	{
		return AutoDetectableTimecodeFormat != EMediaIOAutoDetectableTimecodeFormat::None;
	}

	return true;
}

void UAjaMediaSource::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAjaMediaSource, AutoDetectableTimecodeFormat))
	{
		if (AutoDetectableTimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::None)
		{
			bUseTimeSynchronization = false;
			bEncodeTimecodeInTexel = false;
			EvaluationType = EMediaIOSampleEvaluationType::PlatformTime;
			bFramelock = false;
		}
	}

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCaptureCardMediaSource, EvaluationType))
	{
		// 'Timecode' evaluation is not allowed if no timecode set
		if (AutoDetectableTimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::None || !bUseTimeSynchronization)
		{
			EvaluationType = (EvaluationType == EMediaIOSampleEvaluationType::Timecode ? EMediaIOSampleEvaluationType::PlatformTime : EvaluationType);
		}

		// 'Latest' evaluation is available in JITR only
		if (!bRenderJIT)
		{
			EvaluationType = (EvaluationType == EMediaIOSampleEvaluationType::Latest ? EMediaIOSampleEvaluationType::PlatformTime : EvaluationType);
		}
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif //WITH_EDITOR

void UAjaMediaSource::PostLoad()
{
	Super::PostLoad();

	AssignDefaultConfiguration();

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (TimecodeFormat_DEPRECATED != EMediaIOTimecodeFormat::None)
	{
		switch (TimecodeFormat_DEPRECATED)
		{
			case EMediaIOTimecodeFormat::LTC:
				AutoDetectableTimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::LTC;
				break;
			case EMediaIOTimecodeFormat::VITC:
				AutoDetectableTimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::VITC;
				break;
			default:
				break;
		}
		TimecodeFormat_DEPRECATED = EMediaIOTimecodeFormat::None;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}


void UAjaMediaSource::AssignDefaultConfiguration()
{
	if (!MediaConfiguration.IsValid())
	{
		const FAjaDeviceProvider DeviceProvider;
		const TArray<FMediaIOConfiguration> Configurations = DeviceProvider.GetConfigurations();
		for (const FMediaIOConfiguration& Configuration : Configurations)
		{
			if (Configuration.bIsInput)
			{
				MediaConfiguration = Configuration;
				bRenderJIT = false;
				break;
			}
		}
	}
}
