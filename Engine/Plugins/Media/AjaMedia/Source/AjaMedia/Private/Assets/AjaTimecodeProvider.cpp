// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaTimecodeProvider.h"

#include "Aja/Aja.h"
#include "AjaMediaPrivate.h"
#include "Misc/App.h"
#include "UObject/VirtualProductionObjectVersion.h"

#define LOCTEXT_NAMESPACE "AjaTimecodeProvider"


//~ IAJASyncChannelCallbackInterface implementation
//--------------------------------------------------------------------
// Those are called from the AJA thread. There's a lock inside AJA to prevent this object from dying while in this thread.
struct UAjaTimecodeProvider::FAJACallback : public AJA::IAJATimecodeChannelCallbackInterface
{
	UAjaTimecodeProvider* Owner;
	FAJACallback(UAjaTimecodeProvider* InOwner)
		: Owner(InOwner)
	{}

	//~ IAJAInputCallbackInterface interface
	virtual void OnInitializationCompleted(bool bSucceed) override
	{
		Owner->State = bSucceed ? ETimecodeProviderSynchronizationState::Synchronized : ETimecodeProviderSynchronizationState::Error;
		if (!bSucceed)
		{
			UE_LOGF(LogAjaMedia, Error, "The initialization of '%ls' failed. The TimecodeProvider won't be synchronized.", *Owner->GetName());
		}
	}
};

//~ UAjaTimecodeProvider implementation
//--------------------------------------------------------------------
UAjaTimecodeProvider::UAjaTimecodeProvider()
	: TimecodeChannel(nullptr)
	, SyncCallback(nullptr)
#if WITH_EDITORONLY_DATA
	, InitializedEngine(nullptr)
	, LastAutoSynchronizeInEditorAppTime(0.0)
#endif
	, State(ETimecodeProviderSynchronizationState::Closed)
{
	DeviceProvider = MakeUnique<FAjaDeviceProvider>();
}

bool UAjaTimecodeProvider::FetchTimecode(FQualifiedFrameTime& OutFrameTime)
{
	if (!TimecodeChannel || (State != ETimecodeProviderSynchronizationState::Synchronized))
	{
		return false;
	}

	AJA::FTimecode NewTimecode;

	if (!TimecodeChannel->GetTimecode(NewTimecode))
	{
		State = ETimecodeProviderSynchronizationState::Error;
		return false;
	}

	// We expect the timecode to be processed in the library. 
	// What we receive will be a "linear" timecode even for frame rates greater than 30.

	FFrameRate FrameRate = bUseDedicatedPin ? LTCConfiguration.LtcFrameRate : TimecodeConfiguration.MediaConfiguration.MediaMode.FrameRate;
	FTimecode Timecode = FAja::ConvertAJATimecode2Timecode(NewTimecode, FrameRate);

	OutFrameTime = FQualifiedFrameTime(Timecode, FrameRate);

	return true;
}

bool UAjaTimecodeProvider::Initialize(class UEngine* InEngine)
{
#if WITH_EDITORONLY_DATA
	InitializedEngine = nullptr;
#endif

	State = ETimecodeProviderSynchronizationState::Closed;

	if (!FAja::IsInitialized())
	{
		State = ETimecodeProviderSynchronizationState::Error;
		UE_LOGF(LogAjaMedia, Warning, "The TimecodeProvider '%ls' can't be initialized. AJA is not initialized on your machine.", *GetName());
		return false;
	}

	if (!FAja::CanUseAJACard())
	{
		State = ETimecodeProviderSynchronizationState::Error;
		UE_LOGF(LogAjaMedia, Warning, "The TimecodeProvider '%ls' can't be initialized because Aja card cannot be used. Are you in a Commandlet? You may override this behavior by launching with -ForceAjaUsage", *GetName());
		return false;
	}

	if (bAutoDetectTimecode && !bUseDedicatedPin && !bUseReferenceIn)
	{
		DeviceProvider->AutoDetectConfiguration(FAjaDeviceProvider::FOnConfigurationAutoDetected::CreateUObject(this, &UAjaTimecodeProvider::OnConfigurationAutoDetected, InEngine));
		return true;
	}
	else
	{
		FString FailureReason;
		if ((bUseDedicatedPin && !LTCConfiguration.IsValid()) || (!bUseDedicatedPin && !TimecodeConfiguration.IsValid()))
		{
			State = ETimecodeProviderSynchronizationState::Error;
			UE_LOGF(LogAjaMedia, Warning, "The TimecodeProvider '%ls' configuration is invalid.", *GetName());
			return false;
		}
	}

	return Initialize_Internal(InEngine, AJA::AJATimecodeChannelOptions(*GetName()));
}

void UAjaTimecodeProvider::Shutdown(class UEngine* InEngine)
{
#if WITH_EDITORONLY_DATA
	InitializedEngine = nullptr;
#endif

	State = ETimecodeProviderSynchronizationState::Closed;
	ReleaseResources();
}

void UAjaTimecodeProvider::BeginDestroy()
{
	ReleaseResources();
	Super::BeginDestroy();
}

void UAjaTimecodeProvider::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FVirtualProductionObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FVirtualProductionObjectVersion::GUID) < FVirtualProductionObjectVersion::SupportAjaLTCPin)
	{
		//When loading an old object, we need to fix up the dedicated pin boolean if reference was checked
		bUseDedicatedPin = bUseReferenceIn;
	}
}

void UAjaTimecodeProvider::PostLoad()
{
	Super::PostLoad();
	
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (VideoConfiguration_DEPRECATED.TimecodeFormat != EMediaIOTimecodeFormat::None)
	{
		TimecodeConfiguration.TimecodeFormat = UE::MediaIO::ToAutoDetectableTimecodeFormat(VideoConfiguration_DEPRECATED.TimecodeFormat);
		VideoConfiguration_DEPRECATED.TimecodeFormat = EMediaIOTimecodeFormat::None;
	}

	if (VideoConfiguration_DEPRECATED.IsValid())
	{
		TimecodeConfiguration.MediaConfiguration = VideoConfiguration_DEPRECATED.MediaConfiguration;
		VideoConfiguration_DEPRECATED.MediaConfiguration = FMediaIOConfiguration();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}


void UAjaTimecodeProvider::ReleaseResources()
{
	if (TimecodeChannel)
	{
		TimecodeChannel->Uninitialize();
		delete TimecodeChannel;
		TimecodeChannel = nullptr;

		check(SyncCallback);
		delete SyncCallback;
		SyncCallback = nullptr;
	}

	if (DeviceProvider)
	{
		DeviceProvider->EndAutoDetectConfiguration();
	}
}

ETickableTickType UAjaTimecodeProvider::GetTickableTickType() const
{
#if WITH_EDITORONLY_DATA && WITH_EDITOR
	return ETickableTickType::Conditional;
#else
	return ETickableTickType::Never;
#endif
}

bool UAjaTimecodeProvider::IsTickable() const
{
	return State == ETimecodeProviderSynchronizationState::Error;
}

void UAjaTimecodeProvider::Tick(float DeltaTime)
{
#if WITH_EDITORONLY_DATA && WITH_EDITOR
	if (State == ETimecodeProviderSynchronizationState::Error)
	{
		ReleaseResources();

		// In Editor only, when not in pie, reinitialized the device
		if (InitializedEngine && !GIsPlayInEditorWorld && GIsEditor)
		{
			const double TimeBetweenAttempt = 1.0;
			if (FApp::GetCurrentTime() - LastAutoSynchronizeInEditorAppTime > TimeBetweenAttempt)
			{
				Initialize(InitializedEngine);
				LastAutoSynchronizeInEditorAppTime = FApp::GetCurrentTime();
			}
		}
	}
#endif
}

void UAjaTimecodeProvider::OnConfigurationAutoDetected(TArray<FAjaDeviceProvider::FMediaIOConfigurationWithTimecodeFormat> InConfigurations, class UEngine* InEngine)
{
	AJA::AJATimecodeChannelOptions Options(*GetName());
	
	bool bConfigurationFound = false;
	for (const FAjaDeviceProvider::FMediaIOConfigurationWithTimecodeFormat& Configuration : InConfigurations)
	{
		if (Configuration.Configuration.MediaConnection.Device.DeviceIdentifier == TimecodeConfiguration.MediaConfiguration.MediaConnection.Device.DeviceIdentifier
			&& Configuration.Configuration.MediaConnection.PortIdentifier == TimecodeConfiguration.MediaConfiguration.MediaConnection.PortIdentifier)
		{
            TimecodeConfiguration.MediaConfiguration = Configuration.Configuration;
            TimecodeConfiguration.TimecodeFormat = UE::MediaIO::ToAutoDetectableTimecodeFormat(Configuration.TimecodeFormat);
			bConfigurationFound = true;
			break;
		}
	}
	
	if (!bConfigurationFound)
	{
		UE_LOGF(LogAjaMedia, Warning, "No configuration was detected for TimecodeProvider '%ls'", *GetName());
		ReleaseResources();
		return;
	}

	Initialize_Internal(InEngine, Options);
}

bool UAjaTimecodeProvider::Initialize_Internal(class UEngine* InEngine, AJA::AJATimecodeChannelOptions InOptions)
{
	check(SyncCallback == nullptr);
	SyncCallback = new FAJACallback(this);

	const int32 DeviceIndex = bUseDedicatedPin ? LTCConfiguration.Device.DeviceIdentifier : TimecodeConfiguration.MediaConfiguration.MediaConnection.Device.DeviceIdentifier;
	if (DeviceIndex == -1)
	{ 
		State = ETimecodeProviderSynchronizationState::Error;
		ReleaseResources();
		return false;
	}

	AJA::AJADeviceOptions DeviceOptions(DeviceIndex);

	InOptions.CallbackInterface = SyncCallback;
	InOptions.bUseDedicatedPin = bUseDedicatedPin;
	InOptions.bAutoDetectFormat = bAutoDetectTimecode;

	if (bUseDedicatedPin)
	{
		InOptions.bReadTimecodeFromReferenceIn = bUseReferenceIn;
		InOptions.LTCSourceIndex = LTCConfiguration.LtcIndex;
		InOptions.LTCFrameRateNumerator = LTCConfiguration.LtcFrameRate.Numerator;
		InOptions.LTCFrameRateDenominator = LTCConfiguration.LtcFrameRate.Denominator;
	}
	else
	{
		InOptions.bReadTimecodeFromReferenceIn = false;
		InOptions.LTCSourceIndex = 0;

		InOptions.ChannelIndex = TimecodeConfiguration.MediaConfiguration.MediaConnection.PortIdentifier;
		InOptions.VideoFormatIndex = TimecodeConfiguration.MediaConfiguration.MediaMode.DeviceModeIdentifier;

		InOptions.TransportType = AJA::ETransportType::TT_SdiSingle;
		{
			const EMediaIOTransportType TransportType = TimecodeConfiguration.MediaConfiguration.MediaConnection.TransportType;
			const EMediaIOQuadLinkTransportType QuadTransportType = TimecodeConfiguration.MediaConfiguration.MediaConnection.QuadTransportType;
			switch (TransportType)
			{
			case EMediaIOTransportType::SingleLink:
				InOptions.TransportType = AJA::ETransportType::TT_SdiSingle;
				break;
			case EMediaIOTransportType::DualLink:
				InOptions.TransportType = AJA::ETransportType::TT_SdiDual;
				break;
			case EMediaIOTransportType::QuadLink:
				InOptions.TransportType = QuadTransportType == EMediaIOQuadLinkTransportType::SquareDivision ? AJA::ETransportType::TT_SdiQuadSQ : AJA::ETransportType::TT_SdiQuadTSI;
				break;
			case EMediaIOTransportType::HDMI:
				InOptions.TransportType = AJA::ETransportType::TT_Hdmi;
				break;
			}
		}

		InOptions.TimecodeFormat = AJA::ETimecodeFormat::TCF_None;
		switch (TimecodeConfiguration.TimecodeFormat)
		{
		case EMediaIOAutoDetectableTimecodeFormat::None:
			InOptions.TimecodeFormat = AJA::ETimecodeFormat::TCF_None;
			break;
		case EMediaIOAutoDetectableTimecodeFormat::LTC:
			InOptions.TimecodeFormat = AJA::ETimecodeFormat::TCF_LTC;
			break;
		case EMediaIOAutoDetectableTimecodeFormat::VITC:
			InOptions.TimecodeFormat = AJA::ETimecodeFormat::TCF_VITC1;
			break;
		default:
			break;
		}
	}

	check(TimecodeChannel == nullptr);
	TimecodeChannel = new AJA::AJATimecodeChannel();

	if (!TimecodeChannel->Initialize(DeviceOptions, InOptions))
	{
		State = ETimecodeProviderSynchronizationState::Error;
		ReleaseResources();
		return false;
	}

#if WITH_EDITORONLY_DATA
	InitializedEngine = InEngine;
#endif

	State = ETimecodeProviderSynchronizationState::Synchronizing;
	return true;
}

#undef LOCTEXT_NAMESPACE
