// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaIOCoreAudioSampleBase.h"
#include "AjaMediaPrivate.h"

/*
 * Implements a media audio sample for AjaMedia.
 */
class FAjaMediaAudioSample
	: public FMediaIOCoreAudioSampleBase
{
	using Super = FMediaIOCoreAudioSampleBase;

public:

	bool Initialize(const AJA::AJAAudioFrameData& InAudioData, FTimespan InTime, const TOptional<FTimecode>& InTimecode)
	{
		return Super::Initialize(
			reinterpret_cast<int32*>(InAudioData.AudioBuffer)
			, InAudioData.AudioBufferSize / sizeof(int32)
			, InAudioData.NumChannels
			, InAudioData.AudioRate
			, InTime
			, InTimecode);
	}

	virtual void* RequestBuffer(uint32 InBufferSize) override
	{
		return Super::RequestBuffer(InBufferSize / sizeof(int32));
	}
};

/*
 * Implements a pool for AJA audio sample objects. 
 */
class FAjaMediaAudioSamplePool : public TMediaObjectPool<FAjaMediaAudioSample> { };
