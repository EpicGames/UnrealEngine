// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaIOCoreBinarySampleBase.h"

#include "AjaMediaPrivate.h"

/*
 * Implements a pool for AJA binary sample objects. 
 */

class FAjaMediaBinarySamplePool : public TMediaObjectPool<FMediaIOCoreBinarySampleBase> { };
