// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaMediaPlayer.h"


#include "Aja/Aja.h"
#include "AjaMediaAudioSample.h"
#include "AjaMediaBinarySample.h"
#include "AjaMediaDefinitions.h"
#include "AjaMediaPrivate.h"
#include "AjaMediaSettings.h"
#include "AjaMediaTextureSample.h"
#include "Async/Async.h"
#include "CaptureCardMediaSource.h"

#include "Engine/Engine.h"
#include "GenlockedCustomTimeStep.h"
#include "MediaIOCoreDeinterlacer.h"
#include "MediaIOCoreEncodeTime.h"
#include "MediaIOCoreFileWriter.h"
#include "MediaIOCoreSamples.h"
#include "MediaIOCoreUtilities.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformProcess.h"
#include "IAjaMediaModule.h"
#include "IMediaEventSink.h"
#include "IMediaOptions.h"

#include "Misc/ScopeLock.h"
#include "RenderCommandFence.h"
#include "Stats/Stats.h"
#include "Styling/SlateStyle.h"



#if WITH_EDITOR
#include "EngineAnalytics.h"
#endif

#define LOCTEXT_NAMESPACE "FAjaMediaPlayer"

DECLARE_CYCLE_STAT(TEXT("AJA MediaPlayer Request frame"), STAT_AJA_MediaPlayer_RequestFrame, STATGROUP_Media);
DECLARE_CYCLE_STAT(TEXT("AJA MediaPlayer Process frame"), STAT_AJA_MediaPlayer_ProcessFrame, STATGROUP_Media);

namespace AjaMediaPlayerConst
{
	static const uint32 ModeNameBufferSize = 64;
	static const int32 ToleratedExtraMaxBufferCount = 2;
}

bool bAjaWriteOutputRawDataCmdEnable = false;
static FAutoConsoleCommand AjaWriteOutputRawDataCmd(
	TEXT("Aja.WriteOutputRawData"),
	TEXT("Write Aja raw output buffer to file."), 
	FConsoleCommandDelegate::CreateLambda([]() { bAjaWriteOutputRawDataCmdEnable = true; })
	);

/* FAjaVideoPlayer structors
 *****************************************************************************/

FAjaMediaPlayer::FAjaMediaPlayer(IMediaEventSink& InEventSink)
	: Super(InEventSink)
	, AudioSamplePool(MakeUnique<FAjaMediaAudioSamplePool>())
	, MetadataSamplePool(MakeUnique<FAjaMediaBinarySamplePool>())
	, TextureSamplePool(MakeUnique<FAjaMediaTextureSamplePool>())
	, EventSink(InEventSink)
	, AjaColorSettings(MakeShared<FNativeMediaSourceColorSettings>())
	, DeviceProvider(MakePimpl<FAjaDeviceProvider>())
{
}


FAjaMediaPlayer::~FAjaMediaPlayer()
{
	Close();
}


/* IMediaPlayer interface
 *****************************************************************************/

/**
 * @EventName MediaFramework.AjaSourceOpened
 * @Trigger Triggered when an Aja media source is opened through a media player.
 * @Type Client
 * @Owner MediaIO Team
 */
bool FAjaMediaPlayer::Open(const FString& Url, const IMediaOptions* Options)
{
	bAddedSynchronizedSamples = false;

	if (!FAja::CanUseAJACard())
	{
		UE_LOGF(LogAjaMedia, Warning, "The AjaMediaPlayer can't open URL '%ls' because Aja card cannot be used. Are you in a Commandlet? You may override this behavior by launching with -ForceAjaUsage", *Url);
		return false;
	}

	if (!Super::Open(Url, Options))
	{
		return false;
	}

	const EMediaIOAutoDetectableTimecodeFormat Timecode = (EMediaIOAutoDetectableTimecodeFormat)(Options->GetMediaOption(AjaMediaOption::TimecodeFormat, (int64)EMediaIOAutoDetectableTimecodeFormat::None));
	const bool bAutoDetectTimecode = Timecode == EMediaIOAutoDetectableTimecodeFormat::Auto;
	const bool bAutoDetectVideoFormat = bAutoDetect;

	if (!bAutoDetectTimecode)
	{
		TimecodeFormat = UE::MediaIO::FromAutoDetectableTimecodeFormat(Timecode);
	}
	if (bAutoDetectTimecode || bAutoDetectVideoFormat)
	{
		DeviceProvider->AutoDetectConfiguration(FAjaDeviceProvider::FOnConfigurationAutoDetected::CreateRaw(this, &FAjaMediaPlayer::OnAutoDetected, Url, Options, bAutoDetectVideoFormat, bAutoDetectTimecode));
		return true;
	}
	else
	{
		AJA::AJAInputOutputChannelOptions AjaOptions(TEXT("MediaPlayer"), Options->GetMediaOption(AjaMediaOption::PortIndex, (int64)0));
		return Open_Internal(Url, Options, AjaOptions);
	}
}

void FAjaMediaPlayer::Close()
{
	AjaThreadNewState = EMediaState::Closed;

	DeviceProvider->EndAutoDetectConfiguration();

	if (InputChannel)
	{
		InputChannel->Uninitialize(); // this may block, until the completion of a callback from IAJAChannelCallbackInterface
		delete InputChannel;
		InputChannel = nullptr;
	}

	AudioSamplePool->Reset();
	MetadataSamplePool->Reset();
	TextureSamplePool->Reset();

	//Disable all our channels from the monitor
	Samples->EnableTimedDataChannels(this, EMediaIOSampleType::None);

	AjaThreadCurrentAncSample.Reset();
	AjaThreadCurrentAncF2Sample.Reset();
	AjaThreadCurrentAudioSample.Reset();
	AjaThreadCurrentTextureSample.Reset();

#if WITH_EDITOR
	TWeakObjectPtr<UObject> WeakSourceObj = MediaSourceObject;
	AsyncTask(ENamedThreads::GameThread, [WeakSourceObj]()
	{
		if (UCaptureCardMediaSource* Source = Cast<UCaptureCardMediaSource>(WeakSourceObj.Get()))
		{
			Source->ClearLastDetectedConfiguration();
		}
	});
#endif

	Super::Close();
}

FGuid FAjaMediaPlayer::GetPlayerPluginGUID() const
{
	static FGuid PlayerPluginGUID(0xfde28f0a, 0xf72c4cb9, 0x9c1358fb, 0x1ae552d9);
	return PlayerPluginGUID;
}


FString FAjaMediaPlayer::GetStats() const
{
	FString Stats;

	Stats += FString::Printf(TEXT("		Input port: %s\n"), *GetUrl());
	Stats += FString::Printf(TEXT("		Frame rate: %s\n"), *VideoFrameRate.ToPrettyText().ToString());
	Stats += FString::Printf(TEXT("		  AJA Mode: %s\n"), *VideoTrackFormat.TypeName);

	Stats += TEXT("\n\n");
	Stats += TEXT("Status\n");
	
	if (bUseFrameTimecode)
	{
		//TODO This is not thread safe.
		Stats += FString::Printf(TEXT("		Newest Timecode: %02d:%02d:%02d:%02d\n"), AjaThreadPreviousFrameTimecode.Hours, AjaThreadPreviousFrameTimecode.Minutes, AjaThreadPreviousFrameTimecode.Seconds, AjaThreadPreviousFrameTimecode.Frames);
	}
	else
	{
		Stats += FString::Printf(TEXT("		Timecode: Not Enabled\n"));
	}

	if (bUseVideo)
	{
		Stats += FString::Printf(TEXT("		Buffered video frames: %d\n"), GetSamples().NumVideoSamples());
	}
	else
	{
		Stats += FString::Printf(TEXT("		Buffered video frames: Not enabled\n"));
	}
	
	if (bUseAudio)
	{
		Stats += FString::Printf(TEXT("		Buffered audio frames: %d\n"), GetSamples().NumAudioSamples());
	}
	else
	{
		Stats += FString::Printf(TEXT("		Buffered audio frames: Not enabled\n"));
	}
	
	Stats += FString::Printf(TEXT("		Frames dropped: %d"), LastFrameDropCount);

	return Stats;
}

#if WITH_EDITOR
const FSlateBrush* FAjaMediaPlayer::GetDisplayIcon() const
{
	return IAjaMediaModule::Get().GetStyle()->GetBrush("AjaMediaIcon");
}
#endif //WITH_EDITOR

void FAjaMediaPlayer::TickFetch(FTimespan DeltaTime, FTimespan Timecode)
{
	Super::TickFetch(DeltaTime, Timecode);
	if (InputChannel && CurrentState == EMediaState::Playing)
	{
		ProcessFrame();
		VerifyFrameDropCount();
	}
}

void FAjaMediaPlayer::TickInput(FTimespan DeltaTime, FTimespan Timecode)
{
	// update player state
	EMediaState NewState = AjaThreadNewState;
	
	if (NewState != CurrentState)
	{
		CurrentState = NewState;
		if (CurrentState == EMediaState::Playing)
		{
			EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);
		}
		else if (NewState == EMediaState::Error)
		{
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
			Close();
		}
	}

	if (CurrentState != EMediaState::Playing)
	{
		return;
	}

	TickTimeManagement();
}


/* FAjaMediaPlayer implementation
 *****************************************************************************/
void FAjaMediaPlayer::ProcessFrame()
{
	if (CurrentState == EMediaState::Playing)
	{
		// No need to lock here. That info is only used for debug information.
		AudioTrackFormat.NumChannels = AjaThreadAudioChannels;
		AudioTrackFormat.SampleRate = AjaThreadAudioSampleRate;
	}
}

void FAjaMediaPlayer::VerifyFrameDropCount()
{
	//Update frame dropped status
	if (bVerifyFrameDropCount)
	{
		uint32 FrameDropCount = AjaThreadFrameDropCount;
		if (FrameDropCount > LastFrameDropCount)
		{
			PreviousFrameDropCount += FrameDropCount - LastFrameDropCount;

			static const int32 NumMaxFrameBeforeWarning = 50;
			if (PreviousFrameDropCount % NumMaxFrameBeforeWarning == 0)
			{
				UE_LOGF(LogAjaMedia, Warning, "Loosing frames on AJA input %ls. The current count is %d.", *GetUrl(), PreviousFrameDropCount);
			}
		}
		else if (PreviousFrameDropCount > 0)
		{
			UE_LOGF(LogAjaMedia, Warning, "Lost %d frames on input %ls. Unreal Engine's frame rate is too slow and the capture card was not able to send the frame(s) to Unreal.", PreviousFrameDropCount, *GetUrl());
			PreviousFrameDropCount = 0;
		}
		LastFrameDropCount = FrameDropCount;

		// JITR pipeline always keeps the sample pool full. So every new sample increments internal drop count.
		// This if-condition is intended to avoid "Lost %d XXX frames on input..." message spam every frame.
		if (!IsJustInTimeRenderingEnabled())
		{
			const int32 CurrentMetadataDropCount = Samples->GetMetadataFrameDropCount();
			int32 DeltaMetadataDropCount = CurrentMetadataDropCount;
			if (CurrentMetadataDropCount >= PreviousMetadataFrameDropCount)
			{
				DeltaMetadataDropCount = CurrentMetadataDropCount - PreviousMetadataFrameDropCount;
			}
			PreviousMetadataFrameDropCount = CurrentMetadataDropCount;
			if (DeltaMetadataDropCount > 0)
			{
				UE_LOGF(LogAjaMedia, Warning, "Lost %d metadata frames on input %ls. Frame rate is either too slow or buffering capacity is too small.", DeltaMetadataDropCount, *GetUrl());
			}

			const int32 CurrentAudioDropCount = Samples->GetAudioFrameDropCount();
			int32 DeltaAudioDropCount = CurrentAudioDropCount;
			if (CurrentAudioDropCount >= PreviousAudioFrameDropCount)
			{
				DeltaAudioDropCount = CurrentAudioDropCount - PreviousAudioFrameDropCount;
			}
			PreviousAudioFrameDropCount = CurrentAudioDropCount;
			if (DeltaAudioDropCount > 0)
			{
				UE_LOGF(LogAjaMedia, Warning, "Lost %d audio frames on input %ls. Frame rate is either too slow or buffering capacity is too small.", DeltaAudioDropCount, *GetUrl());
			}

			const int32 CurrentVideoDropCount = Samples->GetVideoFrameDropCount();
			int32 DeltaVideoDropCount = CurrentVideoDropCount;
			if (CurrentVideoDropCount >= PreviousVideoFrameDropCount)
			{
				DeltaVideoDropCount = CurrentVideoDropCount - PreviousVideoFrameDropCount;
			}
			PreviousVideoFrameDropCount = CurrentVideoDropCount;
			if (DeltaVideoDropCount > 0)
			{
				UE_LOGF(LogAjaMedia, Warning, "Lost %d video frames on input %ls. Frame rate is either too slow or buffering capacity is too small.", DeltaVideoDropCount, *GetUrl());
			}
		}
	}
}


/* IAJAInputOutputCallbackInterface implementation
// This is called from the AJA thread. There's a lock inside AJA to prevent this object from dying while in this thread.
*****************************************************************************/
void FAjaMediaPlayer::OnInitializationCompleted(bool bSucceed)
{
	if (bSucceed)
	{
		LastFrameDropCount = InputChannel->GetFrameDropCount();
	}
	AjaThreadNewState = bSucceed ? EMediaState::Playing : EMediaState::Error;
}


void FAjaMediaPlayer::OnCompletion(bool bSucceed)
{
	if (bAutoDetect)
	{
		AjaThreadNewState = EMediaState::Paused; 
	}
	else
	{
		AjaThreadNewState = bSucceed ? EMediaState::Closed : EMediaState::Error;
	}
}

void FAjaMediaPlayer::OnFormatChange(AJA::FAJAVideoFormat Format)
{
	const AJA::AJAVideoFormats::VideoFormatDescriptor Descriptor = AJA::AJAVideoFormats::GetVideoFormat(Format);

	FMediaIOMode MediaIOMode;
	MediaIOMode.DeviceModeIdentifier = Format;
	MediaIOMode.FrameRate = VideoFrameRate;
	MediaIOMode.Resolution = VideoTrackFormat.Dim;

	EMediaIOStandardType Standard = EMediaIOStandardType::Progressive;
	if (Descriptor.bIsInterlacedStandard)
	{
		Standard = EMediaIOStandardType::Interlaced;
	}
	else if (Descriptor.bIsPsfStandard)
	{
		Standard = EMediaIOStandardType::ProgressiveSegmentedFrame;
	}

	MediaIOMode.Standard = Standard;
	VideoTrackFormat.TypeName = MediaIOMode.GetModeName().ToString();

	AJA::AJAInputOutputChannelOptions Options = InputChannel->GetOptions();
	AJA::AJADeviceOptions DeviceOptions = InputChannel->GetDeviceOptions();

	UE_LOGF(LogAjaMedia, Display, "New configuration was detected for AjaMediaPlayer: %ls", *VideoTrackFormat.TypeName);

	Close();
	AsyncTask(ENamedThreads::GameThread, [this, DeviceOptions, Options]()
	{
		OpenFromFormatChange(DeviceOptions, Options);
	});
}

bool FAjaMediaPlayer::OnRequestInputBuffer(const AJA::AJARequestInputBufferData& InRequestBuffer, AJA::AJARequestedInputBufferData& OutRequestedBuffer)
{

	SCOPE_CYCLE_COUNTER(STAT_AJA_MediaPlayer_RequestFrame);

	// Do not request a video buffer if the frame is interlaced. We need 2 samples and we need to process them.
	//We would be able when we have a de-interlacer on the GPU.

	if (AjaThreadNewState != EMediaState::Playing)
	{
		return false;
	}

	// Anc Field 1
	if (bUseAncillary && InRequestBuffer.AncBufferSize > 0)
	{
		AjaThreadCurrentAncSample = MetadataSamplePool->AcquireShared();
		OutRequestedBuffer.AncBuffer = reinterpret_cast<uint8_t*>(AjaThreadCurrentAncSample->RequestBuffer(InRequestBuffer.AncBufferSize));
	}

	// Anc Field 2
	if (bUseAncillary && InRequestBuffer.AncF2BufferSize > 0)
	{
		AjaThreadCurrentAncF2Sample = MetadataSamplePool->AcquireShared();
		OutRequestedBuffer.AncF2Buffer = reinterpret_cast<uint8_t*>(AjaThreadCurrentAncF2Sample->RequestBuffer(InRequestBuffer.AncF2BufferSize));
	}

	// Audio
	if (bUseAudio && InRequestBuffer.AudioBufferSize > 0)
	{
		AjaThreadCurrentAudioSample = AudioSamplePool->AcquireShared();
		OutRequestedBuffer.AudioBuffer = reinterpret_cast<uint8_t*>(AjaThreadCurrentAudioSample->RequestBuffer(InRequestBuffer.AudioBufferSize));
	}

	// Video
	if (bUseVideo && InRequestBuffer.VideoBufferSize > 0 && InRequestBuffer.bIsProgressivePicture)
	{
		// JITR pipeline uses a single texture for GPU texture transfer as a shared one, so it's always available.
		const bool bIsJustInTimeRenderingEnabled = IsJustInTimeRenderingEnabled();
		const bool bCanUseGPUTextureTransfer     = CanUseGPUTextureTransfer();
		if (!bIsJustInTimeRenderingEnabled && bCanUseGPUTextureTransfer)
		{
			const bool bHasTextureAvailableForGPUTransfer = HasTextureAvailableForGPUTransfer();
			if (!bHasTextureAvailableForGPUTransfer)
			{
				UE_LOGF(LogAjaMedia, Error, "No texture available while doing a gpu texture transfer.");
				return false;
			}
		}

		AjaThreadCurrentTextureSample = TextureSamplePool->AcquireShared();
		AjaThreadCurrentTextureSample->RequestBuffer(InRequestBuffer.VideoBufferSize);
		AjaThreadCurrentTextureSample->SetColorConversionSettings(OCIOSettings);

		if (bIsJustInTimeRenderingEnabled && bCanUseGPUTextureTransfer)
		{
			AjaThreadCurrentTextureSample->SetAwaitingForGPUTransfer();
		}

		OutRequestedBuffer.VideoBuffer = reinterpret_cast<uint8_t*>(AjaThreadCurrentTextureSample->GetMutableBuffer());
	}

	return true;
}

bool FAjaMediaPlayer::OnInputFrameReceived(const AJA::AJAInputFrameData& InInputFrame, const AJA::AJAAncillaryFrameData& InAncillaryFrame, const AJA::AJAAudioFrameData& InAudioFrame, const AJA::AJAVideoFrameData& InVideoFrame)
{
	SCOPE_CYCLE_COUNTER(STAT_AJA_MediaPlayer_ProcessFrame);

	if ((AjaThreadNewState != EMediaState::Playing) && (AjaThreadNewState != EMediaState::Paused))
	{
		return false;
	}

	AjaThreadNewState = bPauseRequested ? EMediaState::Paused : EMediaState::Playing;

	AjaThreadFrameDropCount = InInputFrame.FramesDropped;

	FTimespan DecodedTime = FTimespan::FromSeconds(GetPlatformSeconds());
	FTimespan DecodedTimeF2 = DecodedTime + FTimespan::FromSeconds(VideoFrameRate.AsInterval());

	TOptional<FTimecode> DecodedTimecode;
	TOptional<FTimecode> DecodedTimecodeF2;
	if (bUseFrameTimecode)
	{
		//We expect the timecode to be processed in the library. What we receive will be a "linear" timecode even for frame rates greater than 30.
		const int32 FrameLimit = InVideoFrame.bIsProgressivePicture ? FMath::RoundToInt(VideoFrameRate.AsDecimal()) : FMath::RoundToInt(VideoFrameRate.AsDecimal()) - 1;
		if ((int32)InInputFrame.Timecode.Frames >= FrameLimit)
		{
			UE_LOGF(LogAjaMedia, Warning, "Input %ls received an invalid Timecode frame number (%d) for the current frame rate (%ls).", *GetUrl(), InInputFrame.Timecode.Frames, *VideoFrameRate.ToPrettyText().ToString());
		}

		DecodedTimecode = FAja::ConvertAJATimecode2Timecode(InInputFrame.Timecode, VideoFrameRate);
		DecodedTimecodeF2 = DecodedTimecode;
		++DecodedTimecodeF2->Frames;

		const FFrameNumber ConvertedFrameNumber = DecodedTimecode.GetValue().ToFrameNumber(VideoFrameRate);
		const double NumberOfSeconds = ConvertedFrameNumber.Value * VideoFrameRate.AsInterval();
		const FTimespan TimecodeDecodedTime = FTimespan::FromSeconds(NumberOfSeconds);
		if (EvaluationType == EMediaIOSampleEvaluationType::Timecode)
		{
			DecodedTime = TimecodeDecodedTime;
			DecodedTimeF2 = TimecodeDecodedTime + FTimespan::FromSeconds(VideoFrameRate.AsInterval());
		}

		//Previous frame Timecode for stats purposes
		AjaThreadPreviousFrameTimecode = InInputFrame.Timecode;

		if (IsTimecodeLogEnabled())
		{
			UE_LOGF(LogAjaMedia, Log, "Input %ls has timecode : %02d:%02d:%02d:%02d", *GetUrl(), InInputFrame.Timecode.Hours, InInputFrame.Timecode.Minutes, InInputFrame.Timecode.Seconds, InInputFrame.Timecode.Frames);
		}
	}

	// Anc Field 1
	if (bUseAncillary && InAncillaryFrame.AncBuffer)
	{
		if (AjaThreadCurrentAncSample.IsValid())
		{
			if (AjaThreadCurrentAncSample->SetProperties(DecodedTime, VideoFrameRate, DecodedTimecode))
			{
				AddMetadataSample(AjaThreadCurrentAncSample.ToSharedRef());
			}
		}
		else
		{
			auto MetaDataSample = MetadataSamplePool->AcquireShared();
			if (MetaDataSample->Initialize(InAncillaryFrame.AncBuffer, InAncillaryFrame.AncBufferSize, DecodedTime, VideoFrameRate, DecodedTimecode))
			{
				AddMetadataSample(MetaDataSample);
			}
		}
	}

	// Anc Field 2
	if (bUseAncillary && InAncillaryFrame.AncF2Buffer && !InVideoFrame.bIsProgressivePicture)
	{
		if (AjaThreadCurrentAncF2Sample.IsValid())
		{
			if (AjaThreadCurrentAncF2Sample->SetProperties(DecodedTimeF2, VideoFrameRate, DecodedTimecodeF2))
			{
				AddMetadataSample(AjaThreadCurrentAncF2Sample.ToSharedRef());
			}
		}
		else
		{
			auto MetaDataSample = MetadataSamplePool->AcquireShared();
			if (MetaDataSample->Initialize(InAncillaryFrame.AncF2Buffer, InAncillaryFrame.AncF2BufferSize, DecodedTimeF2, VideoFrameRate, DecodedTimecodeF2))
			{
				AddMetadataSample(MetaDataSample);
			}
		}
	}

	// Audio
	if (bUseAudio && InAudioFrame.AudioBuffer)
	{
		if (AjaThreadCurrentAudioSample.IsValid())
		{
			if (AjaThreadCurrentAudioSample->SetProperties(InAudioFrame.AudioBufferSize / sizeof(int32), InAudioFrame.NumChannels, InAudioFrame.AudioRate, DecodedTime, DecodedTimecode))
			{
				AddAudioSample(AjaThreadCurrentAudioSample.ToSharedRef());
			}

			AjaThreadAudioChannels = AjaThreadCurrentAudioSample->GetChannels();
			AjaThreadAudioSampleRate = AjaThreadCurrentAudioSample->GetSampleRate();
		}
		else
		{
			auto AudioSample = AudioSamplePool->AcquireShared();
			if (AudioSample->Initialize(InAudioFrame, DecodedTime, DecodedTimecode))
			{
				AddAudioSample(AudioSample);
			}

			AjaThreadAudioChannels = AudioSample->GetChannels();
			AjaThreadAudioSampleRate = AudioSample->GetSampleRate();
		}
	}

	// Video
	if (bUseVideo && InVideoFrame.VideoBuffer)
	{
		EMediaTextureSampleFormat VideoSampleFormat = EMediaTextureSampleFormat::CharBGRA;
		EMediaIOCoreEncodePixelFormat EncodePixelFormat = EMediaIOCoreEncodePixelFormat::CharBGRA;
		FString OutputFilename;

		switch (InVideoFrame.PixelFormat)
		{
		case AJA::EPixelFormat::PF_8BIT_ARGB:
			VideoSampleFormat = EMediaTextureSampleFormat::CharBGRA;
			EncodePixelFormat = EMediaIOCoreEncodePixelFormat::CharBGRA;
			OutputFilename = TEXT("Aja_Output_8_RGBA");
			break;
		case AJA::EPixelFormat::PF_8BIT_YCBCR:
			VideoSampleFormat = EMediaTextureSampleFormat::CharUYVY;
			EncodePixelFormat = EMediaIOCoreEncodePixelFormat::CharUYVY;
			OutputFilename = TEXT("Aja_Output_8_YUV");
			break;
		case AJA::EPixelFormat::PF_10BIT_RGB:
			VideoSampleFormat = EMediaTextureSampleFormat::CharBGR10A2;
			EncodePixelFormat = EMediaIOCoreEncodePixelFormat::A2B10G10R10;
			OutputFilename = TEXT("Aja_Output_10_RGBA");
			break;
		case AJA::EPixelFormat::PF_10BIT_YCBCR:
			VideoSampleFormat = EMediaTextureSampleFormat::YUVv210;
			EncodePixelFormat = EMediaIOCoreEncodePixelFormat::YUVv210;
			OutputFilename = TEXT("Aja_Output_10_YUV");
			break;
		}

		if (bEncodeTimecodeInTexel && DecodedTimecode.IsSet() && InVideoFrame.bIsProgressivePicture)
		{
			FTimecode SetTimecode = DecodedTimecode.GetValue();
			FMediaIOCoreEncodeTime EncodeTime(EncodePixelFormat, InVideoFrame.VideoBuffer, InVideoFrame.Stride, InVideoFrame.Width, InVideoFrame.Height);
			EncodeTime.Render(SetTimecode.Hours, SetTimecode.Minutes, SetTimecode.Seconds, SetTimecode.Frames);
		}

		if (bAjaWriteOutputRawDataCmdEnable)
		{
			MediaIOCoreFileWriter::WriteRawFile(
				OutputFilename, 
				reinterpret_cast<uint8*>(InVideoFrame.VideoBuffer), 
				InVideoFrame.Stride * InVideoFrame.Height,
				false // bAppend
			);

			bAjaWriteOutputRawDataCmdEnable = false;
		}

		FAjaMediaHDROptions HDROptions = UE::AjaMedia::MakeAjaMediaHDROptions(InVideoFrame.HDROptions);
		ensure(AjaColorSettings.IsValid());

		const UE::Color::EEncoding SourceEncoding = SourceColorSettings.IsValid() ? SourceColorSettings->GetEncodingOverride() : UE::Color::EEncoding::None;
		if (SourceEncoding != UE::Color::EEncoding::None)
		{
			// Explicit user encoding override: propagate both encoding and its scoped
			// reference white so the user's UI selection reaches the sample.
			AjaColorSettings->SetEncodingOverride(SourceEncoding);
			AjaColorSettings->SetReferenceWhiteOverride(SourceColorSettings->GetReferenceWhiteOverride());
		}
		else
		{
			const FTimespan TimeBetweenLogs = FTimespan::FromSeconds(5);

			switch (HDROptions.EOTF)
			{
			case EAjaHDRMetadataEOTF::SDR:
				AjaColorSettings->SetEncodingOverride(UE::Color::EEncoding::sRGB); // Missing support for BT.1886 so we default to sRGB instead.
				break;
			case EAjaHDRMetadataEOTF::PQ:
				AjaColorSettings->SetEncodingOverride(UE::Color::EEncoding::ST2084);
				break;
			case EAjaHDRMetadataEOTF::HLG:
				AjaColorSettings->SetEncodingOverride(UE::Color::EEncoding::HLG);
				break;
			default:
				checkNoEntry();
			}
			// User left encoding at None: reset any leftover reference white so the
			// plugin default (BT.2408 fallback in the sample for HDR) applies cleanly.
			AjaColorSettings->SetReferenceWhiteOverride(UE::Color::EReferenceWhite::None);
		}

		if (SourceColorSettings.IsValid() && SourceColorSettings->HasColorSpaceOverride())
		{
			AjaColorSettings->SetColorSpaceOverride(SourceColorSettings->GetColorSpaceOverride(UE::Color::FColorSpace::GetSRGB()));
		}
		else
		{
			switch (HDROptions.Gamut)
			{
			case EAjaHDRMetadataGamut::Rec709:
				AjaColorSettings->SetColorSpaceOverride(UE::Color::FColorSpace::GetSRGB());
				break;
			case EAjaHDRMetadataGamut::Rec2020:
				AjaColorSettings->SetColorSpaceOverride(UE::Color::FColorSpace::GetRec2020());
				break;
			default:
				checkNoEntry();
			}
		}

		if (AjaThreadCurrentTextureSample.IsValid())
		{
			if (AjaThreadCurrentTextureSample->SetProperties(InVideoFrame.Stride, InVideoFrame.Width, InVideoFrame.Height, VideoSampleFormat, DecodedTime, VideoFrameRate, DecodedTimecode, AjaColorSettings))
			{
				AddVideoSample(AjaThreadCurrentTextureSample.ToSharedRef());
			}
		}
		else
		{
			bool bEvenField = GFrameCounter % 2 == 0;
			if (bEvenField && !InVideoFrame.bIsProgressivePicture && !bAddedSynchronizedSamples && UGenlockedCustomTimeStep::CVarExperimentalFieldFlipFix.GetValueOnAnyThread())
			{
				// Input must be received on an odd frame count or it flips the fields.
				if (UGenlockedCustomTimeStep* TimeStep = Cast<UGenlockedCustomTimeStep>(GEngine->GetCustomTimeStep()))
				{
					if (TimeStep->GetSynchronizationState() == ECustomTimeStepSynchronizationState::Synchronized)
					{
						// Reinitialize genlock if it it was started on the wrong frame (Must be started so the frames are received on odd frames.
						AsyncTask(ENamedThreads::GameThread, [TimeStep]() {
							TimeStep->Shutdown(GEngine);
							TimeStep->Initialize(GEngine);
							});
					}
				}
			}
			else
			{
				if (InVideoFrame.bIsProgressivePicture)
				{
					auto TextureSample = TextureSamplePool->AcquireShared();
					if (TextureSample->InitializeProgressive(InVideoFrame, VideoSampleFormat, DecodedTime, VideoFrameRate, DecodedTimecode, AjaColorSettings))
					{
						TextureSample->SetColorConversionSettings(OCIOSettings);
						AddVideoSample(TextureSample);
					}
				}
				else
				{
					if (UGenlockedCustomTimeStep* TimeStep = Cast<UGenlockedCustomTimeStep>(GEngine->GetCustomTimeStep()))
					{
						bAddedSynchronizedSamples = TimeStep->GetSynchronizationState() == ECustomTimeStepSynchronizationState::Synchronized;
					}

					UE::MediaIOCore::FVideoFrame FrameInfo
					{
						InVideoFrame.VideoBuffer,
						InVideoFrame.VideoBufferSize,
						InVideoFrame.Stride,
						InVideoFrame.Width,
						InVideoFrame.Height,
						VideoSampleFormat,
						DecodedTime,
						VideoFrameRate,
						DecodedTimecode,
						AjaColorSettings
					};

					const TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> DeinterlacedSamples = Deinterlace(FrameInfo);

					for (const TSharedRef<FMediaIOCoreTextureSampleBase>& TextureSample : DeinterlacedSamples)
					{
						TextureSample->SetColorConversionSettings(OCIOSettings);
						AddVideoSample(TextureSample);
					}
				}
			}
		}

		AjaThreadCurrentAncSample.Reset();
		AjaThreadCurrentAncF2Sample.Reset();
		AjaThreadCurrentAudioSample.Reset();
		AjaThreadCurrentTextureSample.Reset();
	}

	return true;
}

bool FAjaMediaPlayer::OnOutputFrameCopied(const AJA::AJAOutputFrameData& InFrameData)
{
	// this is not supported
	check(false);
	return false;
}

bool FAjaMediaPlayer::IsHardwareReady() const
{
	return (AjaThreadNewState == EMediaState::Playing) || (AjaThreadNewState == EMediaState::Paused);
}

void FAjaMediaPlayer::SetupSampleChannels()
{
	FMediaIOSamplingSettings VideoSettings = BaseSettings;
	VideoSettings.BufferSize = MaxNumVideoFrameBuffer;
	Samples->InitializeVideoBuffer(VideoSettings);

	FMediaIOSamplingSettings AudioSettings = BaseSettings;
	AudioSettings.BufferSize = MaxNumAudioFrameBuffer;
	Samples->InitializeAudioBuffer(AudioSettings);

	FMediaIOSamplingSettings MetadataSettings = BaseSettings;
	MetadataSettings.BufferSize = MaxNumMetadataFrameBuffer;
	Samples->InitializeMetadataBuffer(MetadataSettings);
}

bool FAjaMediaPlayer::Open_Internal(const FString& Url, const IMediaOptions* Options, AJA::AJAInputOutputChannelOptions AjaOptions)
{
	AJA::AJADeviceOptions DeviceOptions(Options->GetMediaOption(AjaMediaOption::DeviceIndex, (int64)0));

	AjaOptions.CallbackInterface = this;
	AjaOptions.bOutput = false;
	{
		const EMediaIOTransportType TransportType = (EMediaIOTransportType)(Options->GetMediaOption(AjaMediaOption::TransportType, (int64)EMediaIOTransportType::SingleLink));
		const EMediaIOQuadLinkTransportType QuadTransportType = (EMediaIOQuadLinkTransportType)(Options->GetMediaOption(AjaMediaOption::QuadTransportType, (int64)EMediaIOQuadLinkTransportType::SquareDivision));
		switch (TransportType)
		{
		case EMediaIOTransportType::SingleLink:
			AjaOptions.TransportType = AJA::ETransportType::TT_SdiSingle;
			break;
		case EMediaIOTransportType::DualLink:
			AjaOptions.TransportType = AJA::ETransportType::TT_SdiDual;
			break;
		case EMediaIOTransportType::QuadLink:
			AjaOptions.TransportType = QuadTransportType == EMediaIOQuadLinkTransportType::SquareDivision ? AJA::ETransportType::TT_SdiQuadSQ : AJA::ETransportType::TT_SdiQuadTSI;
			break;
		case EMediaIOTransportType::HDMI:
			AjaOptions.TransportType = AJA::ETransportType::TT_Hdmi;
			break;
		}
	}
	{
		bUseFrameTimecode = TimecodeFormat != EMediaIOTimecodeFormat::None;
		AjaOptions.TimecodeFormat = AJA::ETimecodeFormat::TCF_None;
		switch (TimecodeFormat)
		{
		case EMediaIOTimecodeFormat::None:
			AjaOptions.TimecodeFormat = AJA::ETimecodeFormat::TCF_None;
			break;
		case EMediaIOTimecodeFormat::LTC:
			AjaOptions.TimecodeFormat = AJA::ETimecodeFormat::TCF_LTC;
			break;
		case EMediaIOTimecodeFormat::VITC:
			AjaOptions.TimecodeFormat = AJA::ETimecodeFormat::TCF_VITC1;
			break;
		default:
			break;
		}
		bEncodeTimecodeInTexel = Options->GetMediaOption(AjaMediaOption::EncodeTimecodeInTexel, false);
	}
	{
		const EAjaMediaAudioChannel AudioChannelOption = (EAjaMediaAudioChannel)(Options->GetMediaOption(AjaMediaOption::AudioChannel, (int64)EAjaMediaAudioChannel::Channel8));
		AjaOptions.NumberOfAudioChannel = (AudioChannelOption == EAjaMediaAudioChannel::Channel8) ? 8 : 6;
	}
	{
		if (!bAutoDetect)
		{
			AjaOptions.VideoFormatIndex = Options->GetMediaOption(AjaMediaOption::AjaVideoFormat, (int64)0);
		}
	}
	{
		AjaColorFormat = (EAjaMediaSourceColorFormat)(Options->GetMediaOption(AjaMediaOption::ColorFormat, (int64)EAjaMediaSourceColorFormat::YUV2_8bit));
		switch (AjaColorFormat)
		{
		case EAjaMediaSourceColorFormat::YUV2_8bit:
			if (AjaOptions.bUseKey)
			{
				AjaOptions.PixelFormat = AJA::EPixelFormat::PF_8BIT_ARGB;
			}
			else
			{
				AjaOptions.PixelFormat = AJA::EPixelFormat::PF_8BIT_YCBCR;
			}
			break;
		case EAjaMediaSourceColorFormat::YUV_10bit:
			if (AjaOptions.bUseKey)
			{
				AjaOptions.PixelFormat = AJA::EPixelFormat::PF_10BIT_RGB;
			}
			else
			{
				AjaOptions.PixelFormat = AJA::EPixelFormat::PF_10BIT_YCBCR;
			}
			break;
		default:
			AjaOptions.PixelFormat = AJA::EPixelFormat::PF_8BIT_ARGB;
			break;
		}

		bIsSRGBInput = Options->GetMediaOption(AjaMediaOption::SRGBInput, false);
	}
	{
		AjaOptions.bUseAncillary = bUseAncillary = Options->GetMediaOption(AjaMediaOption::CaptureAncillary, false);
		AjaOptions.bUseAudio = bUseAudio = Options->GetMediaOption(AjaMediaOption::CaptureAudio, false);
		AjaOptions.bUseVideo = bUseVideo = Options->GetMediaOption(AjaMediaOption::CaptureVideo, true);
		AjaOptions.bUseAutoCirculating = Options->GetMediaOption(AjaMediaOption::CaptureWithAutoCirculating, true);
		AjaOptions.bUseKey = false;
		AjaOptions.bBurnTimecode = false;
		AjaOptions.BurnTimecodePercentY = 80;
		AjaOptions.bAutoDetectFormat = bAutoDetect;
		AjaOptions.bStopOnTimeout = Options->GetMediaOption(AjaMediaOption::StopOnTimeout, true);

		//Adjust supported sample types based on what's being captured
		SupportedSampleTypes = AjaOptions.bUseVideo ? EMediaIOSampleType::Video : EMediaIOSampleType::None;
		SupportedSampleTypes |= AjaOptions.bUseAudio ? EMediaIOSampleType::Audio : EMediaIOSampleType::None;
		SupportedSampleTypes |= AjaOptions.bUseAncillary ? EMediaIOSampleType::Metadata : EMediaIOSampleType::None;
		Samples->EnableTimedDataChannels(this, SupportedSampleTypes);
	}

	bVerifyFrameDropCount = Options->GetMediaOption(AjaMediaOption::LogDropFrame, true);
	MaxNumAudioFrameBuffer = Options->GetMediaOption(AjaMediaOption::MaxAudioFrameBuffer, (int64)8);
	MaxNumMetadataFrameBuffer = Options->GetMediaOption(AjaMediaOption::MaxAncillaryFrameBuffer, (int64)8);
	MaxNumVideoFrameBuffer = Options->GetMediaOption(AjaMediaOption::MaxVideoFrameBuffer, (int64)8);

	check(InputChannel == nullptr);
	InputChannel = new AJA::AJAInputChannel();
	if (!InputChannel->Initialize(DeviceOptions, AjaOptions))
	{
		UE_LOGF(LogAjaMedia, Warning, "The AJA port couldn't be opened.");
		CurrentState = EMediaState::Error;
		AjaThreadNewState = EMediaState::Error;
		delete InputChannel;
		InputChannel = nullptr;
	}

	// Setup our different supported channels based on source settings
	SetupSampleChannels();

	// configure format information for base class
	AudioTrackFormat.BitsPerSample = 32;
	AudioTrackFormat.NumChannels = 0;
	AudioTrackFormat.SampleRate = 48000;
	AudioTrackFormat.TypeName = FString(TEXT("PCM"));

	// finalize
	CurrentState = EMediaState::Preparing;
	AjaThreadNewState = EMediaState::Preparing;
	EventSink.ReceiveMediaEvent(EMediaEvent::MediaConnecting);

#if WITH_EDITOR
	RecordSourceOpenedEvent();
#endif

	return true;
}

void FAjaMediaPlayer::OpenFromFormatChange(AJA::AJADeviceOptions DeviceOptions, AJA::AJAInputOutputChannelOptions AjaOptions)
{
	{
		//Adjust supported sample types based on what's being captured
		SupportedSampleTypes = AjaOptions.bUseVideo ? EMediaIOSampleType::Video : EMediaIOSampleType::None;
		SupportedSampleTypes |= AjaOptions.bUseAudio ? EMediaIOSampleType::Audio : EMediaIOSampleType::None;
		SupportedSampleTypes |= AjaOptions.bUseAncillary ? EMediaIOSampleType::Metadata : EMediaIOSampleType::None;
		Samples->EnableTimedDataChannels(this, SupportedSampleTypes);
	}

	check(InputChannel == nullptr);
	InputChannel = new AJA::AJAInputChannel();
	if (!InputChannel->Initialize(DeviceOptions, AjaOptions))
	{
		UE_LOGF(LogAjaMedia, Warning, "The AJA port couldn't be opened.");
		CurrentState = EMediaState::Error;
		AjaThreadNewState = EMediaState::Error;
		delete InputChannel;
		InputChannel = nullptr;
	}

	// Setup our different supported channels based on source settings
	SetupSampleChannels();

	// Configure format information for base class

	// Finalize
	CurrentState = EMediaState::Preparing;
	AjaThreadNewState = EMediaState::Preparing;
	EventSink.ReceiveMediaEvent(EMediaEvent::MediaConnecting);
}

void FAjaMediaPlayer::OnAutoDetected(TArray<FAjaDeviceProvider::FMediaIOConfigurationWithTimecodeFormat> Configurations, FString Url, const IMediaOptions* Options, bool bAutoDetectVideoFormat, bool bAutoDetectTimecodeFormat)
{
	bool bConfigurationFound = false;
	const int64 PortIndex = Options->GetMediaOption(AjaMediaOption::PortIndex, (int64)0);
	const int64 DeviceIndex = Options->GetMediaOption(AjaMediaOption::DeviceIndex, (int64)0);
	AJA::AJAInputOutputChannelOptions AjaOptions(TEXT("MediaPlayer"), PortIndex);

	for (const FAjaDeviceProvider::FMediaIOConfigurationWithTimecodeFormat& Configuration : Configurations)
	{
		if (Configuration.Configuration.MediaConnection.Device.DeviceIdentifier == DeviceIndex
			&& Configuration.Configuration.MediaConnection.PortIdentifier == PortIndex)
		{
			bConfigurationFound = true;
			if (bAutoDetectVideoFormat)
			{
				VideoFrameRate = Configuration.Configuration.MediaMode.FrameRate;
				FIntPoint Resolution = Configuration.Configuration.MediaMode.Resolution;

				VideoTrackFormat.Dim = Resolution;
				VideoTrackFormat.FrameRates = TRange<float>(VideoFrameRate.AsDecimal());
				VideoTrackFormat.FrameRate = VideoFrameRate.AsDecimal();
				VideoTrackFormat.TypeName = Configuration.Configuration.MediaMode.GetModeName().ToString();
			
				AjaOptions.VideoFormatIndex = Configuration.Configuration.MediaMode.DeviceModeIdentifier;
				UE_LOGF(LogAjaMedia, Display, "New configuration was detected for AjaMediaPlayer: %ls", *VideoTrackFormat.TypeName);

#if WITH_EDITOR
				PublishAutoDetectedFormatToSource(Configuration.Configuration);
#endif
			}

			if (bAutoDetectTimecodeFormat)
			{
				TimecodeFormat = Configuration.TimecodeFormat;
			}
			
			//Setup base sampling settings
			BaseSettings.FrameRate = VideoFrameRate;
			break;
		}
	}

	if (!bConfigurationFound)
	{
		UE_LOGF(LogAjaMedia, Warning, "No configuration was detected for MediaPlayer.");
		Close();
		return;
	}

	Open_Internal(Url, Options, AjaOptions);

	NotifyVideoFormatDetected();
}

#if WITH_EDITOR
void FAjaMediaPlayer::PublishAutoDetectedFormatToSource(const FMediaIOConfiguration& InDetectedConfiguration)
{
	TWeakObjectPtr<UObject> WeakSourceObj = MediaSourceObject;
	FMediaIOConfiguration Detected = InDetectedConfiguration;
	AsyncTask(ENamedThreads::GameThread, [WeakSourceObj, Detected]()
	{
		if (UCaptureCardMediaSource* Source = Cast<UCaptureCardMediaSource>(WeakSourceObj.Get()))
		{
			Source->SetLastDetectedConfiguration(Detected);
		}
	});
}
#endif //WITH_EDITOR

bool FAjaMediaPlayer::SetRate(float Rate)
{
	if (FMath::IsNearlyEqual(Rate, 1.0f))
	{
		bPauseRequested = false;
		return true;
	}

	if (FMath::IsNearlyEqual(Rate, 0.0f))
	{
		bPauseRequested = true;
		return true;
	}

	return false;
}

#if WITH_EDITOR
bool FAjaMediaPlayer::ShouldAutoRecordSourceOpenedEvent() const
{
	// AJA can open asynchronously, so it will call RecordSourceOpenedEvent manually
	return false;
}
#endif

TSharedPtr<FMediaIOCoreTextureSampleBase> FAjaMediaPlayer::AcquireTextureSample_AnyThread() const
{
	return TextureSamplePool->AcquireShared();
}

#if WITH_EDITOR
FString FAjaMediaPlayer::GetAnalyticsEventPrefix() const
{
	return TEXT("MediaFramework.Aja");
}

void FAjaMediaPlayer::GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const
{
	FMediaIOCorePlayerBase::GetAnalyticsEventAttributes(EventName, InOutAttributes);
	
	if (InputChannel)
	{
		const int64 DeviceIndex = InputChannel->GetDeviceOptions().DeviceIndex;
		if (FMediaIOConfiguration* Configuration = DeviceProvider->GetConfigurations().FindByPredicate([DeviceIndex](const FMediaIOConfiguration& InConfiguration)
		{
			return InConfiguration.MediaConnection.Device.DeviceIdentifier == DeviceIndex;
		}))
		{
			InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Device"), Configuration->MediaConnection.Device.DeviceName));
			
			const FString SourceStr = FString::Format(TEXT("{0} {1}"), {
				DeviceProvider->GetTransportName(Configuration->MediaConnection.TransportType, Configuration->MediaConnection.QuadTransportType).ToString(),
				Configuration->MediaConnection.PortIdentifier });
			InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Source"), SourceStr));
			InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Standard"), UEnum::GetValueAsString(Configuration->MediaMode.Standard)));
		}
	}

	if (AjaColorSettings->HasColorSpaceOverride())
	{
		const UE::Color::FColorSpace& ColorSpace = AjaColorSettings->GetColorSpaceOverride(UE::Color::FColorSpace::GetSRGB());
		FVector2d Red, Green, Blue, White;
		ColorSpace.GetChromaticities(Red, Green, Blue, White);
		const FString ColorSpaceStr = FString::Format(TEXT("R:{0} G:{1} B:{2} W:{3}"), { Red.ToString(), Green.ToString(), Blue.ToString(), White.ToString() });
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ColorSpaceOverride"), ColorSpaceStr));
	}
	
	if (AjaColorSettings->HasEncodingOverride())
	{
		UE::Color::EEncoding Encoding = AjaColorSettings->GetEncodingOverride();
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Encoding"), UEnum::GetValueAsString((ETextureSourceEncoding)Encoding)));
	}
}
#endif
#undef LOCTEXT_NAMESPACE
