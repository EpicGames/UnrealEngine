// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaIOCorePlayerBase.h"

#include "AjaDeviceProvider.h"
#include "AjaMediaPrivate.h"
#include "AjaMediaSource.h"
#include "HAL/CriticalSection.h"
#include "Templates/PimplPtr.h"

#include <atomic>

class FAjaMediaAudioSample;
class FAjaMediaAudioSamplePool;
class FAjaMediaBinarySamplePool;
class FAjaMediaTextureSample;
class FAjaMediaTextureSamplePool;
class FMediaIOCoreBinarySampleBase;
struct FSlateBrush;
class IMediaEventSink;

enum class EMediaTextureSampleFormat;
enum class EMediaIOSampleType;

namespace AJA
{
	class AJAInputChannel;
}


/**
 * Implements a media player using AJA.
 *
 * The processing of metadata and video frames is delayed until the fetch stage
 * (TickFetch) in order to increase the window of opportunity for receiving AJA
 * frames for the current render frame time code.
 *
 * Depending on whether the media source enables time code synchronization,
 * the player's current play time (CurrentTime) is derived either from the
 * time codes embedded in AJA frames or from the Engine's global time code.
 */
class FAjaMediaPlayer
	: public FMediaIOCorePlayerBase
	, protected AJA::IAJAInputOutputChannelCallbackInterface
{
	using Super = FMediaIOCorePlayerBase;
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InEventSink The object that receives media events from this player.
	 */
	FAjaMediaPlayer(IMediaEventSink& InEventSink);

	/** Virtual destructor. */
	virtual ~FAjaMediaPlayer();

public:

	//~ IMediaPlayer interface

	virtual void Close() override;
	virtual FGuid GetPlayerPluginGUID() const override;

	virtual bool Open(const FString& Url, const IMediaOptions* Options) override;

	virtual void TickFetch(FTimespan DeltaTime, FTimespan Timecode) override;
	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override;

	virtual FString GetStats() const override;

	bool SetRate(float Rate) override;

#if WITH_EDITOR
	virtual bool ShouldAutoRecordSourceOpenedEvent() const override;
#endif

	//~ ITimedDataInput interface
#if WITH_EDITOR
	virtual const FSlateBrush* GetDisplayIcon() const override;
#endif

protected:

	//~ IAJAInputOutputCallbackInterface interface
	
	virtual void OnInitializationCompleted(bool bSucceed) override;
	virtual bool OnRequestInputBuffer(const AJA::AJARequestInputBufferData& InRequestBuffer, AJA::AJARequestedInputBufferData& OutRequestedBuffer) override;
	virtual bool OnInputFrameReceived(const AJA::AJAInputFrameData& InInputFrame, const AJA::AJAAncillaryFrameData& InAncillaryFrame, const AJA::AJAAudioFrameData& AudioFrame, const AJA::AJAVideoFrameData& VideoFrame) override;
	virtual bool OnOutputFrameCopied(const AJA::AJAOutputFrameData& InFrameData) override;
	virtual void OnCompletion(bool bSucceed) override;
	virtual void OnFormatChange(AJA::FAJAVideoFormat Format) override;


protected:

	/**
	 * Process pending audio and video frames, and forward them to the sinks.
	 */
	void ProcessFrame();
	
protected:

	/** Verify if we lost some frames since last Tick*/
	void VerifyFrameDropCount();

	virtual bool IsHardwareReady() const override;

	//~ Begin FMediaIOCorePlayerBase interface
	virtual void SetupSampleChannels() override;
	virtual uint32 GetNumVideoFrameBuffers() const override
	{
		return MaxNumVideoFrameBuffer;
	}
	virtual EMediaIOCoreColorFormat GetColorFormat() const override
	{
		return AjaColorFormat == EAjaMediaSourceColorFormat::YUV2_8bit ? EMediaIOCoreColorFormat::YUV8 : EMediaIOCoreColorFormat::YUV10;
	}

	virtual TSharedPtr<FMediaIOCoreTextureSampleBase> AcquireTextureSample_AnyThread() const override;
	
#if WITH_EDITOR
	virtual FString GetAnalyticsEventPrefix() const override;
	virtual void GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const override;
#endif
	//~ End FMediaIOCorePlayerBase interface

private:
	bool Open_Internal(const FString& Url, const IMediaOptions* Options, AJA::AJAInputOutputChannelOptions AjaOptions);
	void OpenFromFormatChange(AJA::AJADeviceOptions DeviceOptions, AJA::AJAInputOutputChannelOptions AjaOptions);
	void OnAutoDetected(TArray<FAjaDeviceProvider::FMediaIOConfigurationWithTimecodeFormat> Configurations, FString Url, const IMediaOptions* Options, bool bAutoDetectVideoFormat, bool bAutoDetectTimecodeFormat);

#if WITH_EDITOR
	/**
	 * Surface the auto-detected configuration on the source asset so the Media Profile editor can
	 * display the actual signal format instead of the stored default. Safe to call from any thread;
	 * the UObject mutation is dispatched onto the game thread.
	 */
	void PublishAutoDetectedFormatToSource(const FMediaIOConfiguration& InDetectedConfiguration);
#endif

private:

	/** Audio, MetaData, Texture  sample object pool. */
	TUniquePtr<FAjaMediaAudioSamplePool> AudioSamplePool;
	TUniquePtr<FAjaMediaBinarySamplePool> MetadataSamplePool;
	TUniquePtr<FAjaMediaTextureSamplePool> TextureSamplePool;

	TSharedPtr<FMediaIOCoreBinarySampleBase, ESPMode::ThreadSafe> AjaThreadCurrentAncSample;
	TSharedPtr<FMediaIOCoreBinarySampleBase, ESPMode::ThreadSafe> AjaThreadCurrentAncF2Sample;
	TSharedPtr<FAjaMediaAudioSample, ESPMode::ThreadSafe> AjaThreadCurrentAudioSample;
	TSharedPtr<FAjaMediaTextureSample, ESPMode::ThreadSafe> AjaThreadCurrentTextureSample;

	/** The media sample cache. */
	int32 MaxNumAudioFrameBuffer    = 0;
	int32 MaxNumMetadataFrameBuffer = 0;
	int32 MaxNumVideoFrameBuffer    = 0;

	/** Current state of the media player. */
	EMediaState AjaThreadNewState = EMediaState::Closed;

	/** The media event handler. */
	IMediaEventSink& EventSink;

	/** Number of audio channels in the last received sample. */
	int32 AjaThreadAudioChannels = 0;

	/** Audio sample rate in the last received sample. */
	int32 AjaThreadAudioSampleRate = 0;

	/** Number of frames drop from the last tick. */
	int32 AjaThreadFrameDropCount = 0;
	int32 PreviousMetadataFrameDropCount = 0;
	int32 PreviousAudioFrameDropCount = 0;
	int32 PreviousVideoFrameDropCount = 0;

	/** Number of frames drop from the last tick. */
	uint32 LastFrameDropCount     = 0;
	uint32 PreviousFrameDropCount = 0;

	/** Whether to use the time code embedded in AJA frames. */
	bool bEncodeTimecodeInTexel = false;

	/** Whether to use the timecode embedded in a frame. */
	bool bUseFrameTimecode = false;

	/** Whether the input is in sRGB and can have a ToLinear conversion. */
	bool bIsSRGBInput = false;

	/** Which field need to be capture. */
	bool bUseAncillary = false;
	bool bUseAudio = false;
	bool bUseVideo = false;
	bool bVerifyFrameDropCount = true;

	/** Maps to the current input Device */
	AJA::AJAInputChannel* InputChannel = nullptr;

	/** Used to flag which sample types we advertise as supported for timed data monitoring */
	EMediaIOSampleType SupportedSampleTypes = EMediaIOSampleType::None;

	/** Previous frame timecode for stats purpose */
	AJA::FTimecode AjaThreadPreviousFrameTimecode;

	/** Flag to indicate that pause is being requested */
	std::atomic<bool> bPauseRequested = false;

	/** AJA media source color format. */
	EAjaMediaSourceColorFormat AjaColorFormat = EAjaMediaSourceColorFormat::YUV2_8bit;

	/** Container for local media source color setting overrides. */
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> AjaColorSettings;

	/** Device provider used to autodetect input format. */
	TPimplPtr<class FAjaDeviceProvider> DeviceProvider;

	/** Autodetected or specified Timecode format. */
	EMediaIOTimecodeFormat TimecodeFormat = EMediaIOTimecodeFormat::None; 

	/** Whether samples have been added to the queue while the genlocked time step is synchronized. Reset when the time step is closed. */
	bool bAddedSynchronizedSamples = false;
};
