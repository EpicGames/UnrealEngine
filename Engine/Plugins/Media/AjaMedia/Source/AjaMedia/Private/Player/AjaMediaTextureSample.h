// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AJALib.h"
#include "MediaIOCoreTextureSampleBase.h"
#include "MediaShaders.h"

/**
 * Implements a media texture sample for AjaMedia.
 */
class FAjaMediaTextureSample
	: public FMediaIOCoreTextureSampleBase
{
	using Super = FMediaIOCoreTextureSampleBase;

public:
	/**
	 * Initialize the sample.
	 *
	 * @param InVideoData The video frame data.
	 * @param InSampleFormat The sample format.
	 * @param InTime The sample time (in the player's own clock).
	 * @param InFrameRate The framerate of the media that produce the sample.
	 * @param InTimecode The sample timecode if available.
	 * @param InSourceColorSettings Information about the texture color encoding and color space.
	 */
	bool InitializeProgressive(const AJA::AJAVideoFrameData& InVideoData, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings)
	{
		return Super::Initialize(InVideoData.VideoBuffer
			, InVideoData.VideoBufferSize
			, InVideoData.Stride
			, InVideoData.Width
			, InVideoData.Height
			, InSampleFormat
			, InTime
			, InFrameRate
			, InTimecode
			, InSourceColorSettings);
	}

	/**
	 * Initialize the sample.
	 *
	 * @param InVideoData The video frame data.
	 * @param InSampleFormat The sample format.
	 * @param InTime The sample time (in the player's own clock).
	 * @param InFrameRate The framerate of the media that produce the sample.
	 * @param InTimecode The sample timecode if available.
	 * @param bEven Only take the even frame from the image.
	 * @param InSourceColorSettings Information about the texture color encoding and color space.
	 */
	bool InitializeInterlaced_Halfed(const AJA::AJAVideoFrameData& InVideoData, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, bool bInEven, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings)
	{
		return Super::InitializeWithEvenOddLine(bInEven
			, InVideoData.VideoBuffer
			, InVideoData.VideoBufferSize
			, InVideoData.Stride
			, InVideoData.Width
			, InVideoData.Height
			, InSampleFormat
			, InTime
			, InFrameRate
			, InTimecode
			, InSourceColorSettings);
	}
};

/*
 * Implements a pool for AJA texture sample objects.
 */
class FAjaMediaTextureSamplePool : public TMediaObjectPool<FAjaMediaTextureSample> { };
