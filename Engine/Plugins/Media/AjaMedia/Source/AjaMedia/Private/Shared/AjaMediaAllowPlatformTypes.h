// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef AJA_PLATFORM_TYPES_GUARD
	#define AJA_PLATFORM_TYPES_GUARD
#else
	#error Nesting AjaAllowPlatformTypes.h is not allowed!
#endif

#ifndef PLATFORM_WINDOWS
	#include "Processing.AJA.compat.h"
#endif

#define DWORD ::DWORD
#define FLOAT ::FLOAT

#ifndef TRUE
	#define TRUE 1
#endif

#ifndef FALSE
	#define FALSE 0
#endif
