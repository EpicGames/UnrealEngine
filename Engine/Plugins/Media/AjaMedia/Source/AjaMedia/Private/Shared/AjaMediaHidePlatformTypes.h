// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Not included directly

#ifdef AJA_PLATFORM_TYPES_GUARD
	#undef AJA_PLATFORM_TYPES_GUARD
#else
	#error Mismatched AjaHidePLatformTypes.h detected.
#endif

#undef DWORD
#undef FLOAT

#ifdef TRUE
	#undef TRUE
#endif

#ifdef FALSE
	#undef FALSE
#endif
