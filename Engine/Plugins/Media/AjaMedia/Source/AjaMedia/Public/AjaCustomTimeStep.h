// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenlockedCustomTimeStep.h"

#include "AJALib.h"
#include "AjaDeviceProvider.h"
#include "MediaIOCoreDefinitions.h"
#include "AjaMediaSource.h"

#include "Misc/Timecode.h"

#include "AjaCustomTimeStep.generated.h"

class UEngine;

/**
 * Control the Engine TimeStep via the AJA card.
 * When the signal is lost in the editor (not in PIE), the CustomTimeStep will try to re-synchronize every second.
 */
UCLASS(Blueprintable, editinlinenew, meta=(DisplayName="AJA SDI Input", MediaIOCustomLayout="AJA"))
class AJAMEDIA_API UAjaCustomTimeStep : public UGenlockedCustomTimeStep
{
	GENERATED_UCLASS_BODY()

public:
	//~ UFixedFrameRateCustomTimeStep interface
	virtual bool Initialize(UEngine* InEngine) override;
	virtual void Shutdown(UEngine* InEngine) override;
	virtual bool UpdateTimeStep(UEngine* InEngine) override;
	virtual ECustomTimeStepSynchronizationState GetSynchronizationState() const override;
	virtual FFrameRate GetFixedFrameRate() const override;

	//~ UGenlockedCustomTimeStep interface
	virtual uint32 GetLastSyncCountDelta() const override;
	virtual bool IsLastSyncDataValid() const override;
	virtual FFrameRate GetSyncRate() const override;
	virtual bool WaitForSync() override;
	virtual bool SupportsFormatAutoDetection() const { return true; };

	//~ UObject interface
	virtual void BeginDestroy() override;
#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

private:
	struct FAJACallback;
	friend FAJACallback;

	void ReleaseResources();
	bool VerifyGenlockSignal();
	bool Initialize_Internal(class UEngine* InEngine);
	void OnConfigurationAutoDetected(TArray<FAjaDeviceProvider::FMediaIOConfigurationWithTimecodeFormat> InConfigurations, class UEngine* InEngine, bool bReinitialize);
	void DetectConfiguration(class UEngine* InEngine, bool bReinitialize);
	bool IsInterlaced() const;

public:
	/**
	 * If true, the Engine will wait for a signal coming in from the Reference In pin.
	 * It will also configure the card Genlock mode and configure the selected Media Port as an output.
	 */
	UPROPERTY(EditAnywhere, Category="Genlock")
	bool bUseReferenceIn;

	/** The device, port and video settings that correspond to where the Genlock signal will be coming from */
	UPROPERTY(EditAnywhere, Category="Genlock", meta=(DisplayName="Configuration"))
	FMediaIOConfiguration MediaConfiguration;

	/**
	 * If this is disabled, genlock will attempt to recover if the SDK timeouts while waiting for an interrupt signal from the card. It will remain in the synchronizing state until the card receives a signal.
	 * Note: In editor outside of PIE and -game, the CustomTimeStep will try to restart itself automatically regardless of this setting.
	 */
	UPROPERTY(EditAnywhere, Category = "Genlock")
	bool bStopOnCardTimeout = true;

	/**
	 * If true, the Engine will wait for the frame to be read.
	 * This will introduce random latency (the time it takes to read a frame).
     * Use this option when you want to synchronize the engine with the incoming frame and discard the buffered frames.
     * @note If false, there is no guarantee that the incoming frame will be ready since it takes some time to read a frame.
     * @note This will not work as intended with interlaced transport because both fields are processed at the same time.
	 */
	UPROPERTY(EditAnywhere, Category = "Genlock options", meta=(EditCondition="!bUseReferenceIn"))
	bool bWaitForFrameToBeReady;

	/** The type of Timecode to read from SDI stream. */
	UPROPERTY(EditAnywhere, Category="Genlock options", meta=(EditCondition="!bUseReferenceIn"))
	EMediaIOTimecodeFormat TimecodeFormat;

	/** Enable mechanism to detect Engine loop overrunning the source */
	UPROPERTY(EditAnywhere, Category="Genlock options", meta=(DisplayName="Display Dropped Frames Warning"))
	bool bEnableOverrunDetection;

private:
	/** AJA Port to capture the Sync */
	AJA::AJASyncChannel* SyncChannel;
	FAJACallback* SyncCallback;

#if WITH_EDITORONLY_DATA
	/** Engine used to initialize the CustomTimeStep */
	UPROPERTY(Transient)
	TObjectPtr<UEngine> InitializedEngine;

	/** When Auto synchronize is enabled, the time the last attempt was triggered. */
	double LastAutoSynchronizeInEditorAppTime;

	double LastAutoDetectInEditorAppTime;
#endif

	/** The current SynchronizationState of the CustomTimeStep */
	ECustomTimeStepSynchronizationState State;

	/** Warn if there is a CustomTimeStep and a vsync at the same time but only once. */
	bool bWarnedAboutVSync;

	/** Remember the last Sync Count*/
	bool bIsPreviousSyncCountValid;
	uint32 PreviousSyncCount;
	uint32 SyncCountDelta;

	/** Remember last detected video format */
	AJA::FAJAVideoFormat LastDetectedVideoFormat;
	bool bLastDetectedVideoFormatInitialized;

	TUniquePtr<class FAjaDeviceProvider> DeviceProvider;
	bool bEncounteredInvalidAutoDetectFrame = false;
	bool bIgnoreWarningForOneFrame = true;
};
