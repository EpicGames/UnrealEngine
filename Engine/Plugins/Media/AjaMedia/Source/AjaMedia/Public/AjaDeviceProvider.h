// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMediaIOCoreDeviceProvider.h"
#include "UObject/ObjectMacros.h"
#include "AjaDeviceProvider.generated.h"


/**
 * Configuration of an AJA timecode from Video
 */
USTRUCT()
struct AJAMEDIA_API FAjaMediaTimecodeConfiguration
{
	GENERATED_BODY()
	
	FAjaMediaTimecodeConfiguration();

	/** Used by the UX to have a default selected value. May not be valid with every device. */
	static FAjaMediaTimecodeConfiguration GetDefault();

public:
	/** Read the timecode from a video signal. */
	UPROPERTY(VisibleAnywhere, Category=AJA)
	FMediaIOConfiguration MediaConfiguration;

	/** Timecode format to read from a video signal. */
	UPROPERTY(VisibleAnywhere, Category=AJA)
	EMediaIOTimecodeFormat TimecodeFormat;

public:

	/** Return true if the configuration has been set properly */
	bool IsValid() const;

	bool operator== (const FAjaMediaTimecodeConfiguration& Other) const;

	/**
	 * Get the configuration text representation.
	 * @return String representation, i.e. "Video/Single1/1080p30fps/LTC".
	 */
	FText ToText() const;
};

/**
 * Configuration of an AJA timecode.
 */
USTRUCT()
struct AJAMEDIA_API FAjaMediaTimecodeReference
{
	GENERATED_BODY()

	FAjaMediaTimecodeReference();

	/** Used by the UX to have a default selected value. May not be valid with every device. */
	static FAjaMediaTimecodeReference GetDefault();

public:
	/** The frame rate of the LTC from the reference pin.*/
	UPROPERTY(VisibleAnywhere, Category=AJA)
	FMediaIODevice Device;

	/** The LTC index to read from the reference pin. */
	UPROPERTY(VisibleAnywhere, Category=AJA)
	int32 LtcIndex;

	/** The frame rate of the LTC from the reference pin.*/
	UPROPERTY(VisibleAnywhere, Category=AJA)
	FFrameRate LtcFrameRate;

public:
	/** Return true if the configuration has been set properly */
	bool IsValid() const;

	bool operator== (const FAjaMediaTimecodeReference& Other) const;

	/** Get the configuration text representation. */
	FText ToText() const;
};

class FAJAAutoDetectChannelCallback;

/**
 * Implementation of IMediaIOCoreDeviceProvider for AJA
 */
class AJAMEDIA_API FAjaDeviceProvider : public IMediaIOCoreDeviceProvider
{
public:
	static FName GetProviderName();
	static FName GetProtocolName();

	FAjaDeviceProvider();
	virtual ~FAjaDeviceProvider();
	FAjaDeviceProvider(const FAjaDeviceProvider&) = delete;
	FAjaDeviceProvider& operator=(const FAjaDeviceProvider&) = delete;

	/** Can device do fill and key */
	bool CanDeviceDoAlpha(const FMediaIODevice& InDevice) const;

	/** Auto detect sources that are currently streaming to the device */
	struct FMediaIOConfigurationWithTimecodeFormat
	{
		FMediaIOConfiguration Configuration;
		EMediaIOTimecodeFormat TimecodeFormat;
	};
	DECLARE_DELEGATE_OneParam(FOnConfigurationAutoDetected, TArray<FMediaIOConfigurationWithTimecodeFormat>);

	/** Auto detect sources that are currently streaming to the device */
	void AutoDetectConfiguration(FOnConfigurationAutoDetected OnAutoDetected);
	void EndAutoDetectConfiguration();


public:
	virtual FName GetFName() override;

	virtual TArray<FMediaIOConnection> GetConnections() const override;
	virtual TArray<FMediaIOConfiguration> GetConfigurations() const override;
	virtual TArray<FMediaIOConfiguration> GetConfigurations(bool bAllowInput, bool bAllowOutput) const override;
	virtual TArray<FMediaIOInputConfiguration> GetInputConfigurations() const override;
	virtual TArray<FMediaIOOutputConfiguration> GetOutputConfigurations() const override;
	virtual TArray<FMediaIODevice> GetDevices() const override;
	virtual TArray<FMediaIOMode> GetModes(const FMediaIODevice& InDevice, bool bInOutput) const override;
	virtual TArray<FMediaIOVideoTimecodeConfiguration> GetTimecodeConfigurations() const override;
	TArray<FAjaMediaTimecodeConfiguration> GetTimecodeConfiguration() const;
	TArray<FAjaMediaTimecodeReference> GetTimecodeReferences() const;

	virtual FMediaIOConfiguration GetDefaultConfiguration() const override;
	virtual FMediaIOMode GetDefaultMode() const override;
	virtual FMediaIOInputConfiguration GetDefaultInputConfiguration() const override;
	virtual FMediaIOOutputConfiguration GetDefaultOutputConfiguration() const override;
	virtual FMediaIOVideoTimecodeConfiguration GetDefaultTimecodeConfiguration() const override;

private:
	TUniquePtr<FAJAAutoDetectChannelCallback> AutoDetectCallback;
};
