// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "AjaMediaDefinitions.generated.h"

namespace AJA
{
	struct FAjaHDROptions;
}

/**
 * HDR Transfer function.
 * Must match AJA::EAjaHDRMetadataEOTF.
 */
UENUM()
enum class EAjaHDRMetadataEOTF : uint8
{
	SDR,
	HLG,
	PQ
};
	
/**
 * HDR Color Gamut.
 * Must match AJA::EAjaHDRMetadataGamut.
 */
UENUM()
enum class EAjaHDRMetadataGamut : uint8
{
	Rec709,
	Rec2020
};

/**
 * Set of metadata describing a HDR video signal.
 */
USTRUCT()
struct AJAMEDIA_API FAjaMediaHDROptions
{
	GENERATED_BODY()

	/** Transfer function to use for converting the video signal to an optical signal. */
	UPROPERTY(EditAnywhere, Category = "HDR")
	EAjaHDRMetadataEOTF EOTF = EAjaHDRMetadataEOTF::SDR;

	/** The color gamut of the video signal. */
	UPROPERTY(EditAnywhere, Category = "HDR")
	EAjaHDRMetadataGamut Gamut = EAjaHDRMetadataGamut::Rec709;
};

namespace UE::AjaMedia
{
	/** Create Aja HDR UStruct from the AJALib's hdr struct. */
	FAjaMediaHDROptions MakeAjaMediaHDROptions(const AJA::FAjaHDROptions& HDROptions);
	/** Create AJALib's hdr struct from the Aja HDR UStruct. */
	AJA::FAjaHDROptions MakeAjaHDROptions(const FAjaMediaHDROptions& HDROptions);
}
