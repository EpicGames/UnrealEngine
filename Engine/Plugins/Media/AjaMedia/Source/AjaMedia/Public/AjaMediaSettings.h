// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AjaMediaSettings.generated.h"

/**
 * Settings for the AjaMedia plug-in.
 */
UCLASS(config=AjaMedia)
class AJAMEDIA_API UAjaMediaSettings : public UObject
{
	GENERATED_BODY()
};
