// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureCardMediaSource.h"
#include "Engine/TextureDefines.h"
#include "MediaIOCoreDefinitions.h"


#include "AjaMediaSource.generated.h"

/**
 * Native data format.
 */
UENUM()
enum class EAjaMediaSourceColorFormat : uint8
{
	YUV2_8bit UMETA(DisplayName = "8bit YUV"),
	YUV_10bit UMETA(DisplayName = "10bit YUV"),
};

/**
 * Available number of audio channel supported by Unreal Engine & AJA
 */
UENUM()
enum class EAjaMediaAudioChannel : uint8
{
	Channel6,
	Channel8,
};

/**
 * Media source for AJA streams.
 */
UCLASS(BlueprintType, hideCategories=(Platforms,Object), PrioritizeCategories=(AJA, Video, ColorManagement, Audio, Ancillary, Synchronization, Debug), meta=(MediaIOCustomLayout="AJA"))
class AJAMEDIA_API UAjaMediaSource : public UCaptureCardMediaSource
{
	GENERATED_BODY()

	/** Default constructor. */
	UAjaMediaSource();

public:
	/** The device, port and video settings that correspond to the input. */
	UPROPERTY(EditAnywhere, Category="AJA", meta=(DisplayName="Configuration"))
	FMediaIOConfiguration MediaConfiguration;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "Use AutoDetectableTimecodeFormat")
	/** Use the time code embedded in the input stream. */
	UPROPERTY(meta=(DeprecatedProperty))
	EMediaIOTimecodeFormat TimecodeFormat_DEPRECATED = EMediaIOTimecodeFormat::None;
#endif

	/** Use the time code embedded in the input stream. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="AJA", meta = (DisplayName = "Timecode Format"))
	EMediaIOAutoDetectableTimecodeFormat AutoDetectableTimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::Auto; 
	
	/**
	 * Use a ring buffer to capture and transfer data.
	 * This may decrease transfer latency but increase stability.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="AJA")
	bool bCaptureWithAutoCirculating;

public:
	/**
	 * Capture Ancillary from the AJA source.
	 * It will decrease performance
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Ancillary")
	bool bCaptureAncillary;

	/** Maximum number of ancillary data frames to buffer. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category="Ancillary", meta=(EditCondition="bCaptureAncillary", ClampMin="1", ClampMax="32"))
	int32 MaxNumAncillaryFrameBuffer;

	/**
	 * If this is disabled, input will keep going even if we hit a timeout when waiting for a vertical interrupt from the card. 
	 */
	UPROPERTY(EditAnywhere, Category = "AJA")
	bool bStopInputOnCardTimeout = true;

public:
	/** Capture Audio from the AJA source. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Audio")
	bool bCaptureAudio;

	/** Desired number of audio channel to capture. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Audio", meta=(EditCondition="bCaptureAudio"))
	EAjaMediaAudioChannel AudioChannel;

	/** Maximum number of audio frames to buffer. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category="Audio", meta=(EditCondition="bCaptureAudio", ClampMin="1", ClampMax="32"))
	int32 MaxNumAudioFrameBuffer;

public:
	/** Capture Video from the AJA source. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Video")
	bool bCaptureVideo;

	/** Native data format internally used by the device after being converted from SDI/HDMI signal. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Video", meta=(EditCondition="bCaptureVideo"))
	EAjaMediaSourceColorFormat ColorFormat;

#if WITH_EDITORONLY_DATA
	/** 
	 * Whether the video input is in sRGB color space.
	 * A sRGB to Linear conversion will be applied resulting in a texture in linear space.
	 * @Note If the texture is not in linear space, it won't look correct in the editor. Another pass will be required either through Composure or other means.
	 */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage = "Use ColorEncoding instead."))
	bool bIsSRGBInput_DEPRECATED = false;
#endif

	/** Maximum number of video frames to buffer. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category="Video", meta=(EditCondition="bCaptureVideo", ClampMin="1", ClampMax="32"))
	int32 MaxNumVideoFrameBuffer;

public:
	/** Log a warning when there's a drop frame. */
	UPROPERTY(EditAnywhere, Category="Debug")
	bool bLogDropFrame;

	/**
	 * Burn Frame Timecode in the input texture without any frame number clipping.
	 * @Note Only supported with progressive format.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Debug", meta=(DisplayName="Burn Frame Timecode"))
	bool bEncodeTimecodeInTexel;


public:
	//~ IMediaOptions interface

	virtual bool GetMediaOption(const FName& Key, bool DefaultValue) const override;
	virtual int64 GetMediaOption(const FName& Key, int64 DefaultValue) const override;
	virtual FString GetMediaOption(const FName& Key, const FString& DefaultValue) const override;
	virtual bool HasMediaOption(const FName& Key) const override;

public:
	//~ UMediaSource interface

	virtual FString GetUrl() const override;
	virtual bool Validate() const override;
	
#if WITH_EDITOR
	virtual FString GetDescriptionString() const override;
	virtual void GetDetailsPanelInfoElements(TArray<TTuple<FText, FText, FText>>& OutInfoElements) const override;
#endif //WITH_EDITOR


public:
	//~ UObject interface
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR

	virtual void PostLoad() override;
	virtual bool SupportsFormatAutoDetection() const override { return true; }
	//~ End UObject interface

private:
	/** Assigns a default configuration if the current configuration is invalid. */
	void AssignDefaultConfiguration();
};
