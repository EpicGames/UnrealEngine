// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenlockedTimecodeProvider.h"
#include "AjaMediaSource.h"
#include "AjaDeviceProvider.h"
#include "MediaIOCoreDefinitions.h"
#include "Tickable.h"

#include "AjaTimecodeProvider.generated.h"

namespace AJA
{
	class AJATimecodeChannel;
	struct AJATimecodeChannelOptions;
}

class UEngine;

/**
 * Class to fetch a timecode via an AJA card.
 * When the signal is lost in the editor (not in PIE), the TimecodeProvider will try to re-synchronize every second.
 */
UCLASS(Blueprintable, editinlinenew, meta=(DisplayName="AJA SDI Input", MediaIOCustomLayout="AJA"))
class AJAMEDIA_API UAjaTimecodeProvider : public UGenlockedTimecodeProvider, public FTickableGameObject
{
public:
	GENERATED_BODY()

	UAjaTimecodeProvider();

	//~ UTimecodeProvider interface
	virtual bool FetchTimecode(FQualifiedFrameTime& OutFrameTime) override;
	virtual ETimecodeProviderSynchronizationState GetSynchronizationState() const override { return State; }
	virtual bool Initialize(class UEngine* InEngine) override;
	virtual void Shutdown(class UEngine* InEngine) override;
	virtual bool SupportsAutoDetected() const override
	{
		return true;
	}
    
	virtual void SetIsAutoDetected(bool bInIsAutoDetected) override
	{
		bAutoDetectTimecode = bInIsAutoDetected;
	}
	
	virtual bool IsAutoDetected() const override
	{
		return bAutoDetectTimecode;
	}

	//~ FTickableGameObject interface
	virtual ETickableTickType GetTickableTickType() const override;
	virtual bool IsTickable() const override;
	virtual bool IsTickableWhenPaused() const override { return true; }
	virtual bool IsTickableInEditor() const override { return true; }
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(UAjaTimecodeProvider, STATGROUP_Tickables); }

	//~ UObject interface
	virtual void BeginDestroy() override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;

private:
	struct FAJACallback;
	friend FAJACallback;

	void ReleaseResources();
	bool Initialize_Internal(class UEngine* InEngine, AJA::AJATimecodeChannelOptions InOptions);
	void OnConfigurationAutoDetected(TArray<FAjaDeviceProvider::FMediaIOConfigurationWithTimecodeFormat> InConfigurations, class UEngine* InEngine);

public:

	/**
	 * Should we read the timecode from a dedicated LTC pin or an SDI input.
	 */
	UPROPERTY(EditAnywhere, Category="Timecode")
	bool bUseDedicatedPin;

	/**
	 * Read LTC timecode from reference pin. Will fail if device doesn't support that feature.
	 */
	UPROPERTY(EditAnywhere, Category="Timecode", meta = (EditCondition = "bUseDedicatedPin"))
	bool bUseReferenceIn;

	/**
	 * Where to read LTC timecode from with which FrameRate expected
	 */
	UPROPERTY(EditAnywhere, Category="Timecode", meta=(EditCondition="bUseDedicatedPin"))
	FAjaMediaTimecodeReference LTCConfiguration;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "Use VideoConfiguration instead")
	/**
     * It read the timecode from an input source.
	 */
	UPROPERTY(meta=(DeprecatedProperty))
	FAjaMediaTimecodeConfiguration VideoConfiguration_DEPRECATED;
#endif
	
	UPROPERTY(EditAnywhere, Category="Timecode", meta=(EditCondition="!bUseDedicatedPin"))
	/** Use the time code embedded in the input stream. */
	/** Timecode format to read from a video signal. */
	FMediaIOVideoTimecodeConfiguration TimecodeConfiguration;
	
	UPROPERTY()
	bool bAutoDetectTimecode = true;

private:
	/** AJA channel associated with reading LTC timecode */
	AJA::AJATimecodeChannel* TimecodeChannel;
	FAJACallback* SyncCallback;

#if WITH_EDITORONLY_DATA
	/** Engine used to initialize the Provider */
	UPROPERTY(Transient)
	TObjectPtr<UEngine> InitializedEngine;

	/** The time the last attempt to auto synchronize was triggered. */
	double LastAutoSynchronizeInEditorAppTime;
#endif

	/** The current SynchronizationState of the TimecodeProvider*/
	ETimecodeProviderSynchronizationState State;

	TUniquePtr<FAjaDeviceProvider> DeviceProvider;
};
