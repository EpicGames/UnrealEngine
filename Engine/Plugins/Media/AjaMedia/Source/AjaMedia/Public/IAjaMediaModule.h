// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class FSlateStyleSet;
class IMediaEventSink;
class IMediaPlayer;

/**
 * Interface for the AjaMedia module.
 */
class IAjaMediaModule : public IModuleInterface
{
public:

	static inline IAjaMediaModule& Get()
	{
		static const FName ModuleName = "AjaMedia";
		return FModuleManager::LoadModuleChecked<IAjaMediaModule>(ModuleName);
	}

	/**
	 * Create an AJA based media player.
	 * @param EventSink The object that receives media events from the player.
	 * @return A new media player, or nullptr if a player couldn't be created.
	 */
	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) = 0;

	/** @return SlateStyleSet to be used across the AjaMedia module */
	virtual TSharedPtr<FSlateStyleSet> GetStyle() = 0;

	/** @return true if the Aja module and AJA dll could be loaded */
	virtual bool IsInitialized() const = 0;

	/** @return true if the Aja card can be used */
	virtual bool CanBeUsed() const = 0;
};

