// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AjaMediaEditor : ModuleRules
	{
		public AjaMediaEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AjaMedia",
					"AjaMediaOutput",
					"AssetDefinition",
					"Core",
					"CoreUObject",
					"EditorFramework",
					"Engine",
					"MediaAssets",
					"MediaIOCore",
					"MediaIOEditor",
					"MediaPlayerEditor",
					"Projects",
					"PropertyEditor",
					"Settings",
					"Slate",
					"SlateCore",
					"TimeManagement",
					"UnrealEd"
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"AssetTools"
				});
		}
	}
}
