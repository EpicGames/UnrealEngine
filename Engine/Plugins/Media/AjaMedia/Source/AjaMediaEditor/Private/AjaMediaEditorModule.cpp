// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaDeviceProvider.h"
#include "AjaMediaOutput.h"
#include "AjaMediaSettings.h"
#include "AjaMediaSource.h"
#include "AjaTimecodeProvider.h"

#include "Customizations/AjaMediaTimecodeReferenceCustomization.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Templates/UniquePtr.h"

#define LOCTEXT_NAMESPACE "AjaMediaEditor"

/**
 * Implements the AjaMediaEditor module.
 */
class FAjaMediaEditorModule : public IModuleInterface
{
public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		RegisterCustomizations();
		RegisterStyle();
	}

	virtual void ShutdownModule() override
	{
		if (UObjectInitialized() && !IsEngineExitRequested())
		{
			UnregisterStyle();
			UnregisterCustomizations();
		}
	}

private:
	TUniquePtr<FSlateStyleSet> StyleInstance;

private:

	/** Register details view customizations. */
	void RegisterCustomizations()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout(FAjaMediaTimecodeReference::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAjaMediaTimecodeReferenceCustomization::MakeInstance));
		
	}

	/** Unregister details view customizations. */
	void UnregisterCustomizations()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout(UAjaMediaSource::StaticClass()->GetFName());
		PropertyModule.UnregisterCustomPropertyTypeLayout(FAjaMediaTimecodeReference::StaticStruct()->GetFName());
	}

	void RegisterStyle()
	{
#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(StyleInstance->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)

		StyleInstance = MakeUnique<FSlateStyleSet>("AjaMediaStyle");

		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("AjaMedia"));
		if (Plugin.IsValid())
		{
			StyleInstance->SetContentRoot(FPaths::Combine(Plugin->GetContentDir(), TEXT("Editor/Icons")));
		}

		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon64x64(64.0f, 64.0f);

		StyleInstance->Set("ClassThumbnail.AjaMediaSource", new IMAGE_BRUSH("AjaMediaSource_64x", Icon64x64));
		StyleInstance->Set("ClassIcon.AjaMediaSource", new IMAGE_BRUSH("AjaMediaSource_20x", Icon20x20));
		StyleInstance->Set("ClassThumbnail.AjaMediaOutput", new IMAGE_BRUSH("AjaMediaOutput_64x", Icon64x64));
		StyleInstance->Set("ClassIcon.AjaMediaOutput", new IMAGE_BRUSH("AjaMediaOutput_20x", Icon20x20));

		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance.Get());

#undef IMAGE_BRUSH
	}

	void UnregisterStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance.Get());
		StyleInstance.Reset();
	}
};


IMPLEMENT_MODULE(FAjaMediaEditorModule, AjaMediaEditor);

#undef LOCTEXT_NAMESPACE

