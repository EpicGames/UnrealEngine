// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AjaMediaOutput.h"
#include "AssetDefinition/AssetDefinition_MediaOutput.h"
#include "AssetDefinition_AjaMediaOutput.generated.h"

UCLASS()
class UAssetDefinition_AjaMediaOutput : public UAssetDefinition_MediaOutput
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_AjaMediaOutput", "AJA Media Output"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAjaMediaOutput::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_AjaMediaOutputSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
