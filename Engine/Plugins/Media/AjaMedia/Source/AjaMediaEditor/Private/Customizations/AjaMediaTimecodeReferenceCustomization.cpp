// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/AjaMediaTimecodeReferenceCustomization.h"

#include "AjaDeviceProvider.h"
#include "CommonFrameRates.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "MediaIOPermutationsSelectorBuilder.h"
#include "ObjectEditorUtils.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMediaPermutationsSelector.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"


#define LOCTEXT_NAMESPACE "AjaMediaTimecodeReferenceCustomization"

namespace AjaMediaTimecodeReferenceCustomization
{
	static const FName NAME_DeviceIndex("DeviceIndex");
	static const FName NAME_LtcIndex("ReferenceLtcIndex");
	static const FName NAME_FrameRate("ReferenceFrameRate");

	struct FMediaTimecodePermutationsSelectorBuilder
	{
		static bool IdenticalProperty(FName ColumnName, const FAjaMediaTimecodeReference& Left, const FAjaMediaTimecodeReference& Right)
		{
			if (ColumnName == NAME_DeviceIndex) return Left.Device.DeviceIdentifier == Right.Device.DeviceIdentifier;
			if (ColumnName == NAME_LtcIndex) return Left.LtcIndex == Right.LtcIndex;
			if (ColumnName == NAME_FrameRate) return Left.LtcFrameRate == Right.LtcFrameRate;
			check(false);
			return false;
		}

		static bool Less(FName ColumnName, const FAjaMediaTimecodeReference& Left, const FAjaMediaTimecodeReference& Right)
		{
			if (ColumnName == NAME_DeviceIndex) return Left.Device.DeviceIdentifier < Right.Device.DeviceIdentifier;
			if (ColumnName == NAME_LtcIndex) return Left.LtcIndex < Right.LtcIndex;
			if (ColumnName == NAME_FrameRate) return Left.LtcFrameRate.AsDecimal() < Right.LtcFrameRate.AsDecimal();
			check(false);
			return false;
		}

		static FText GetLabel(FName ColumnName, const FAjaMediaTimecodeReference& Item)
		{
			if (ColumnName == NAME_DeviceIndex) return FText::FromName(Item.Device.DeviceName);
			if (ColumnName == NAME_LtcIndex) return FText::AsNumber(Item.LtcIndex);
			if (ColumnName == NAME_FrameRate) return Item.LtcFrameRate.ToPrettyText();
			check(false);
			return FText::GetEmpty();
		}

		static FText GetTooltip(FName ColumnName, const FAjaMediaTimecodeReference& Item)
		{
			if (ColumnName == NAME_DeviceIndex) return FText::FromString(FString::Printf(TEXT("%s as index: %d"), *Item.Device.DeviceName.ToString(), Item.Device.DeviceIdentifier));
			if (ColumnName == NAME_LtcIndex) return LOCTEXT("ReferenceLtcIndexTooltip", "The LTC index to read from the reference pin.");
			if (ColumnName == NAME_FrameRate)
			{
				if (const FCommonFrameRateInfo* Found = FCommonFrameRates::Find(Item.LtcFrameRate))
				{
					return Found->Description;
				}
				return Item.LtcFrameRate.ToPrettyText();
			}
			check(false);
			return FText::GetEmpty();
		}
	};
}

TAttribute<FText> FAjaMediaTimecodeReferenceCustomization::GetContentText()
{
	FAjaMediaTimecodeReference* Value = GetPropertyValueFromPropertyHandle<FAjaMediaTimecodeReference>();
	return MakeAttributeLambda([=] { return Value->ToText(); });
}

TSharedRef<SWidget> FAjaMediaTimecodeReferenceCustomization::HandleSourceComboButtonMenuContent()
{
	PermutationSelector.Reset();

	SelectedConfiguration = *GetPropertyValueFromPropertyHandle<FAjaMediaTimecodeReference>();

	TArray<FAjaMediaTimecodeReference> MediaConfigurations = FAjaDeviceProvider().GetTimecodeReferences();
	if (MediaConfigurations.Num() == 0)
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("NoConfigurationFound", "No configuration found"));
	}

	using TSelection = SMediaPermutationsSelector<FAjaMediaTimecodeReference, AjaMediaTimecodeReferenceCustomization::FMediaTimecodePermutationsSelectorBuilder>;
	TSharedRef<TSelection> Selector = SNew(TSelection)
		.PermutationsSource(MoveTemp(MediaConfigurations))
		.SelectedPermutation(SelectedConfiguration)
		.OnSelectionChanged(this, &FAjaMediaTimecodeReferenceCustomization::OnSelectionChanged)
		.OnButtonClicked(this, &FAjaMediaTimecodeReferenceCustomization::OnButtonClicked)
		+ TSelection::Column(AjaMediaTimecodeReferenceCustomization::NAME_DeviceIndex)
		.Label(LOCTEXT("DeviceLabel", "Device"))
		+ TSelection::Column(AjaMediaTimecodeReferenceCustomization::NAME_LtcIndex)
		.Label(LOCTEXT("LtcIndexLabel", "LTC Index"))
		+ TSelection::Column(AjaMediaTimecodeReferenceCustomization::NAME_FrameRate)
		.Label(LOCTEXT("FrameRateLabel", "Frame Rate"));
	PermutationSelector = Selector;

	return Selector;
}

void FAjaMediaTimecodeReferenceCustomization::OnSelectionChanged(FAjaMediaTimecodeReference SelectedItem)
{
	SelectedConfiguration = SelectedItem;
}

FReply FAjaMediaTimecodeReferenceCustomization::OnButtonClicked() const
{
	AssignValue(SelectedConfiguration);

	TSharedPtr<SWidget> SharedPermutationSelector = PermutationSelector.Pin();
	if (SharedPermutationSelector.IsValid())
	{
		TSharedRef<SWindow> ParentContextMenuWindow = FSlateApplication::Get().FindWidgetWindow(SharedPermutationSelector.ToSharedRef()).ToSharedRef();
		FSlateApplication::Get().RequestDestroyWindow(ParentContextMenuWindow);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
