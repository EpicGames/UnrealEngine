// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/MediaIOCustomizationBase.h"

#include "AjaDeviceProvider.h"
#include "Input/Reply.h"

/**
 * Implements a details view customization for the FAjaMediaTimecodeReferenceConfiguration
 */
class FAjaMediaTimecodeReferenceCustomization : public FMediaIOCustomizationBase
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() { return MakeShareable(new FAjaMediaTimecodeReferenceCustomization); }

private:
	virtual TAttribute<FText> GetContentText() override;
	virtual TSharedRef<SWidget> HandleSourceComboButtonMenuContent() override;

	void OnSelectionChanged(FAjaMediaTimecodeReference SelectedItem);
	FReply OnButtonClicked() const;

	TWeakPtr<SWidget> PermutationSelector;
	FAjaMediaTimecodeReference SelectedConfiguration;
};
