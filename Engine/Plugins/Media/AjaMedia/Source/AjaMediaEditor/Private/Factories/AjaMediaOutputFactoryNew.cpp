// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaMediaOutputFactoryNew.h"

#include "AssetTypeCategories.h"
#include "AjaMediaOutput.h"
#include "EngineAnalytics.h"


/* UAjaMediaSourceFactoryNew structors
 *****************************************************************************/

UAjaMediaOutputFactoryNew::UAjaMediaOutputFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAjaMediaOutput::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

/**
 * @EventName MediaFramework.AjaMediaOutputCreated
 * @Trigger Triggered when an Aja media output asset is created.
 * @Type Client
 * @Owner MediaIO Team
 */
UObject* UAjaMediaOutputFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.AjaMediaOutputCreated"));
	}
	
	return NewObject<UAjaMediaOutput>(InParent, InClass, InName, Flags);
}


uint32 UAjaMediaOutputFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UAjaMediaOutputFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}
