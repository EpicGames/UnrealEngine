// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "AjaMediaOutputFactoryNew.generated.h"


/**
 * Implements a factory for UAjaMediaOutput objects.
 */
UCLASS(hidecategories=Object)
class UAjaMediaOutputFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	//~ UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
};
