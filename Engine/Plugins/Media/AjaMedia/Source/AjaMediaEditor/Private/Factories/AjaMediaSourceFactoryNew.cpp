// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaMediaSourceFactoryNew.h"

#include "AssetTypeCategories.h"
#include "AjaMediaSource.h"


/* UAjaMediaSourceFactoryNew structors
 *****************************************************************************/

UAjaMediaSourceFactoryNew::UAjaMediaSourceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAjaMediaSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

UObject* UAjaMediaSourceFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAjaMediaSource>(InParent, InClass, InName, Flags);
}


uint32 UAjaMediaSourceFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UAjaMediaSourceFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}
