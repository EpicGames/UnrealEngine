// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AjaMediaFactory : ModuleRules
	{
		public AjaMediaFactory(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MediaAssets",
					"Projects",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
					"AjaMedia",
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
				});

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				DynamicallyLoadedModuleNames.Add("AjaMedia");
			}
		}
	}
}
