// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AjaMediaOutput : ModuleRules
	{
		public AjaMediaOutput(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"AjaMedia",
					"MediaIOCore",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AjaCore",
					"Core",
					"CoreUObject",
					"Engine",
					"GPUTextureTransfer",
					"MovieSceneCapture",
					"RenderCore",
					"RHI",
					"Slate",
					"SlateCore",
					"TimeManagement"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"EditorFramework",
						"MainFrame",
						"UnrealEd"
					}
				);
			}
		}
	}
}
