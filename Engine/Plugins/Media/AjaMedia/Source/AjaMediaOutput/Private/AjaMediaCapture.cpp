// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaMediaCapture.h"

#include "AJALib.h"
#include "AjaDeviceProvider.h"
#include "AjaMediaOutput.h"
#include "AjaMediaOutputModule.h"
#include "GPUTextureTransferModule.h"
#include "Engine/Engine.h"
#include "HAL/Event.h"
#include "HAL/IConsoleManager.h"
#include "IAjaMediaOutputModule.h"
#include "IAjaMediaModule.h"
#include "IMediaIOCoreModule.h"
#include "MediaIOCoreDefinitions.h"
#include "MediaIOCoreFileWriter.h"
#include "MediaIOCoreSubsystem.h"
#include "Misc/ScopeLock.h"
#include "Slate/SceneViewport.h"
#include "Widgets/SViewport.h"

#if WITH_EDITOR
#include "AnalyticsEventAttribute.h"
#include "Editor.h"
#include "EngineAnalytics.h"
#include "Interfaces/IMainFrameModule.h"
#endif

static FAutoConsoleVariableDeprecated CVarAjaEnableGPUDirect_Deprecated(TEXT("Aja.EnableGPUDirect"), TEXT("MediaIO.EnableGPUDirect"), TEXT("5.6"));

/* namespace AjaMediaCaptureDevice
*****************************************************************************/
namespace AjaMediaCaptureDevice
{
	struct FAjaMediaEncodeOptions
	{
		FAjaMediaEncodeOptions(const int32 InWidth, const int32 InHeight, const EAjaMediaOutputPixelFormat InEncodePixelFormat, bool bInUseKey)
		{
			switch (InEncodePixelFormat)
			{
			case EAjaMediaOutputPixelFormat::PF_8BIT_YUV:
				if (bInUseKey)
				{
					Initialize(InWidth * 4, InWidth, EMediaIOCoreEncodePixelFormat::CharBGRA, TEXT("Aja_Input_8_RGBA"));
					break;
				}
				else
				{
					Initialize(InWidth * 4, InWidth * 2, EMediaIOCoreEncodePixelFormat::CharUYVY, TEXT("Aja_Input_8_YUV"));
					break;
				}
			case EAjaMediaOutputPixelFormat::PF_10BIT_YUV:
				if (bInUseKey)
				{
					Initialize(InWidth * 4, InWidth, EMediaIOCoreEncodePixelFormat::A2B10G10R10, TEXT("Aja_Input_10_RGBA"));
					break;
				}
				else
				{
					Initialize(InWidth * 16, InWidth * 6, EMediaIOCoreEncodePixelFormat::YUVv210, TEXT("Aja_Input_10_YUV"));
					break;
				}
			default:
				checkNoEntry();
			}
		}

		void Initialize(const uint32 InStride, const uint32 InTimeEncodeWidth, const EMediaIOCoreEncodePixelFormat InEncodePixelFormat, const FString InOutputFilename)
		{
			Stride = InStride;
			TimeEncodeWidth = InTimeEncodeWidth;
			EncodePixelFormat = InEncodePixelFormat;
			OutputFilename = InOutputFilename;
		}

		uint32 Stride;
		uint32 TimeEncodeWidth;
		EMediaIOCoreEncodePixelFormat EncodePixelFormat;
		FString OutputFilename;
	};
	
	AJA::FTimecode ConvertToAJATimecode(const FTimecode& InTimecode, float InEngineFPS, float InAjaFPS)
	{
		const float Divider = InEngineFPS / InAjaFPS;

		AJA::FTimecode Timecode;
		Timecode.Hours = InTimecode.Hours;
		Timecode.Minutes = InTimecode.Minutes;
		Timecode.Seconds = InTimecode.Seconds;
		Timecode.Frames = int32(float(InTimecode.Frames) / Divider);
		return Timecode;
	}
}

namespace AjaMediaCaptureUtils
{
	AJA::FAjaHDROptions MakeAjaHDRMetadata(const FAjaMediaHDROptions& HDROptions)
	{
		AJA::FAjaHDROptions HDRMetadata;
		
		HDRMetadata.Gamut = (AJA::EAjaHDRMetadataGamut) HDROptions.Gamut;
		HDRMetadata.EOTF = (AJA::EAjaHDRMetadataEOTF) HDROptions.EOTF;

		return HDRMetadata;
	}
	
	AJA::ETransportType ConvertTransportType(const EMediaIOTransportType TransportType, const EMediaIOQuadLinkTransportType QuadTransportType)
	{
		switch (TransportType)
		{
		case EMediaIOTransportType::SingleLink:
			return AJA::ETransportType::TT_SdiSingle;
		case EMediaIOTransportType::DualLink:
			return AJA::ETransportType::TT_SdiDual;
		case EMediaIOTransportType::QuadLink:
			return QuadTransportType == EMediaIOQuadLinkTransportType::SquareDivision ? AJA::ETransportType::TT_SdiQuadSQ : AJA::ETransportType::TT_SdiQuadTSI;
		case EMediaIOTransportType::HDMI:
			return AJA::ETransportType::TT_Hdmi;
		default:
			checkNoEntry();
			return AJA::ETransportType::TT_SdiSingle;
		}
	}
	
	AJA::EPixelFormat ConvertPixelFormat(EAjaMediaOutputPixelFormat PixelFormat, bool bUseKey)
	{
		switch (PixelFormat)
		{
		case EAjaMediaOutputPixelFormat::PF_8BIT_YUV:
			return bUseKey ? AJA::EPixelFormat::PF_8BIT_ARGB : AJA::EPixelFormat::PF_8BIT_YCBCR;
		case EAjaMediaOutputPixelFormat::PF_10BIT_YUV:
			return bUseKey ? AJA::EPixelFormat::PF_10BIT_RGB : AJA::EPixelFormat::PF_10BIT_YCBCR;
		default:
			return AJA::EPixelFormat::PF_8BIT_YCBCR;
		}
	}
	
	AJA::ETimecodeFormat ConvertTimecode(EMediaIOTimecodeFormat TimecodeFormat)
	{
		switch (TimecodeFormat)
		{
		case EMediaIOTimecodeFormat::None:
			return AJA::ETimecodeFormat::TCF_None;
		case EMediaIOTimecodeFormat::LTC:
			return AJA::ETimecodeFormat::TCF_LTC;
		case EMediaIOTimecodeFormat::VITC:
			return AJA::ETimecodeFormat::TCF_VITC1;
		default:
			return AJA::ETimecodeFormat::TCF_None;
		}
	}

	AJA::EAJAReferenceType Convert(EMediaIOReferenceType OutputReference)
	{
		switch(OutputReference)
		{
		case EMediaIOReferenceType::External:
			return AJA::EAJAReferenceType::EAJA_REFERENCETYPE_EXTERNAL;
		case EMediaIOReferenceType::Input:
			return AJA::EAJAReferenceType::EAJA_REFERENCETYPE_INPUT;
		default:
			return AJA::EAJAReferenceType::EAJA_REFERENCETYPE_FREERUN;
		}
	}
}

bool bAjaWriteInputRawDataCmdEnable = false;
static FAutoConsoleCommand AjaWriteInputRawDataCmd(
	TEXT("Aja.WriteInputRawData"),
	TEXT("Write Aja raw input buffer to file."),
	FConsoleCommandDelegate::CreateLambda([]() { bAjaWriteInputRawDataCmdEnable = true; })
	);

///* FAjaOutputCallback definition
//*****************************************************************************/
struct UAjaMediaCapture::FAjaOutputCallback : public AJA::IAJAInputOutputChannelCallbackInterface
{
	virtual void OnInitializationCompleted(bool bSucceed) override;
	virtual bool OnRequestInputBuffer(const AJA::AJARequestInputBufferData& RequestBuffer, AJA::AJARequestedInputBufferData& OutRequestedBuffer) override;
	virtual bool OnInputFrameReceived(const AJA::AJAInputFrameData& InInputFrame, const AJA::AJAAncillaryFrameData& InAncillaryFrame, const AJA::AJAAudioFrameData& AudioFrame, const AJA::AJAVideoFrameData& VideoFrame) override;
	virtual bool OnOutputFrameCopied(const AJA::AJAOutputFrameData& InFrameData) override;
	virtual void OnOutputFrameStarted() override;
	virtual void OnCompletion(bool bSucceed) override;
	UAjaMediaCapture* Owner;

	/** Last frame drop count to detect count */
	uint64 LastFrameDropCount = 0;
	uint64 PreviousDroppedCount = 0;
};

///* FAjaOutputCallback definition
//*****************************************************************************/
struct UAjaMediaCapture::FAJAOutputChannel : public AJA::AJAOutputChannel
{
	FAJAOutputChannel() = default;
};

///* UAjaMediaCapture implementation
//*****************************************************************************/
UAjaMediaCapture::UAjaMediaCapture()
	: bWaitForSyncEvent(false)
	, bLogDropFrame(false)
	, bEncodeTimecodeInTexel(false)
	, PixelFormat(EAjaMediaOutputPixelFormat::PF_8BIT_YUV)
	, UseKey(false)
	, bSavedIgnoreTextureAlpha(false)
	, bIgnoreTextureAlphaChanged(false)
	, FrameRate(30, 1)
	, WakeUpEvent(nullptr)
{
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		if (FGPUTextureTransferModule::Get().IsInitialized())
		{
			TextureTransfer = FGPUTextureTransferModule::Get().GetTextureTransfer();
		}

#if WITH_EDITOR
		if (GEditor)
		{
			// In editor, an asset re-save dialog can prevent AJA from cleaning up in the regular PreExit callback,
			// So we have to do our cleanup before the regular callback is called.
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			CanCloseEditorDelegateHandle = MainFrame.RegisterCanCloseEditor(IMainFrameModule::FMainFrameCanCloseEditor::CreateUObject(this, &UAjaMediaCapture::CleanupPreEditorExit));
		}
#else
		FCoreDelegates::OnEnginePreExit.AddUObject(this, &UAjaMediaCapture::OnEnginePreExit);
#endif
	}
}

UAjaMediaCapture::~UAjaMediaCapture()
{
#if WITH_EDITOR
	if (IMainFrameModule* MainFrame = FModuleManager::GetModulePtr<IMainFrameModule>("MainFrame"))
	{
		MainFrame->UnregisterCanCloseEditor(CanCloseEditorDelegateHandle);
	}
#else
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
#endif
}

bool UAjaMediaCapture::ValidateMediaOutput() const
{
	UAjaMediaOutput* AjaMediaOutput = Cast<UAjaMediaOutput>(MediaOutput);
	if (!AjaMediaOutput)
	{
		UE_LOGF(LogAjaMediaOutput, Error, "Can not start the capture. MediaSource's class is not supported.");
		return false;
	}

	return true;
}

bool UAjaMediaCapture::InitializeCapture()
{
	UAjaMediaOutput* AjaMediaSource = CastChecked<UAjaMediaOutput>(MediaOutput);

	if (FGPUTextureTransferModule::Get().IsEnabled() && !FGPUTextureTransferModule::Get().IsInitialized())
	{
		// This will trigger an initialization of GPUDirect.
		TextureTransfer = FGPUTextureTransferModule::Get().GetTextureTransfer();
	}
	
	const bool bResult = InitAJA(AjaMediaSource);

	if (GEngine)
	{
		GEngine->GetEngineSubsystem<UMediaIOCoreSubsystem>()->OnBufferReceived_AudioThread().AddUObject(this, &UAjaMediaCapture::OnAudioBufferReceived_AudioThread);
	}
	
	return bResult;
}

bool UAjaMediaCapture::PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	ApplyViewportTextureAlpha(InSceneViewport);
	return true;
}

bool UAjaMediaCapture::UpdateSceneViewportImpl(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	RestoreViewportTextureAlpha(GetCapturingSceneViewport());
	ApplyViewportTextureAlpha(InSceneViewport);
	return true;
}

bool UAjaMediaCapture::UpdateRenderTargetImpl(UTextureRenderTarget2D* InRenderTarget)
{
	RestoreViewportTextureAlpha(GetCapturingSceneViewport());
	return true;
}

bool UAjaMediaCapture::UpdateMediaTextureImpl(UMediaTexture* InMediaTexture)
{
	RestoreViewportTextureAlpha(GetCapturingSceneViewport());
	return true;
}

bool UAjaMediaCapture::UpdateAudioDeviceImpl(const FAudioDeviceHandle& InAudioDeviceHandle)
{
	return CreateAudioOutput(InAudioDeviceHandle, Cast<UAjaMediaOutput>(MediaOutput));
}

void UAjaMediaCapture::StopCaptureImpl(bool bAllowPendingFrameToBeProcess)
{
	if (!bAllowPendingFrameToBeProcess)
	{
		StopCapture_Future();
	}
}

TFuture<void> UAjaMediaCapture::StopCapture_Future()
{
	FScopeLock ScopeLock(&CopyingCriticalSection);

	TFuture<void> Future;

	if (OutputChannel)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAjaMediaCapture::StopCapture_Future);

		TPromise<void> CompletionPromise;
		Future = CompletionPromise.GetFuture();

		if (TexturesToRelease.Num())
		{
			ENQUEUE_RENDER_COMMAND(AjaMediaCaptureInitialize)(
			[this, TexturesToUnregister = TexturesToRelease](FRHICommandListImmediate& RHICmdList) mutable
			{
				// Unregistering them from the game thread would cause a crash in the GPUDirect SDK.
				for (FTextureRHIRef& Texture : TexturesToUnregister)
				{
					TextureTransfer->UnregisterTexture(Texture->GetTexture2D());
				}
			});

			TexturesToRelease.Reset();
		}
		
		OutputChannel->Uninitialize(MoveTemp(CompletionPromise));
	}

	if (!Future.IsValid())
	{
		Future = MakeFulfilledPromise<void>().GetFuture();
	}

	return Future;
}

bool UAjaMediaCapture::ShouldCaptureRHIResource() const
{
	return TextureTransfer && FGPUTextureTransferModule::Get().IsEnabled();
}

bool UAjaMediaCapture::ShouldCaptureThisFrame(const FCaptureBaseData& InBaseData) const
{
	if (OutputChannel)
	{
		return OutputChannel->ShouldCaptureThisFrame(InBaseData.SourceFrameTimecode, InBaseData.SourceFrameNumber);
	}
	return false;
}

void UAjaMediaCapture::ApplyViewportTextureAlpha(TSharedPtr<FSceneViewport> InSceneViewport)
{
	if (InSceneViewport.IsValid())
	{
		TSharedPtr<SViewport> Widget(InSceneViewport->GetViewportWidget().Pin());
		if (Widget.IsValid())
		{
			bSavedIgnoreTextureAlpha = Widget->GetIgnoreTextureAlpha();

			UAjaMediaOutput* AjaMediaSource = CastChecked<UAjaMediaOutput>(MediaOutput);
			if (AjaMediaSource->OutputConfiguration.OutputType == EMediaIOOutputType::FillAndKey)
			{
				if (bSavedIgnoreTextureAlpha)
				{
					bIgnoreTextureAlphaChanged = true;
					Widget->SetIgnoreTextureAlpha(false);
				}
			}
		}
	}
}

void UAjaMediaCapture::RestoreViewportTextureAlpha(TSharedPtr<FSceneViewport> InSceneViewport)
{
	// restore the ignore texture alpha state
	if (bIgnoreTextureAlphaChanged)
	{
		if (InSceneViewport.IsValid())
		{
			TSharedPtr<SViewport> Widget(InSceneViewport->GetViewportWidget().Pin());
			if (Widget.IsValid())
			{
				Widget->SetIgnoreTextureAlpha(bSavedIgnoreTextureAlpha);
			}
		}
		bIgnoreTextureAlphaChanged = false;
	}
}

bool UAjaMediaCapture::CleanupPreEditorExit()
{
	OnEnginePreExit();
	return true;
}

void UAjaMediaCapture::OnEnginePreExit()
{
	if (OutputChannel)
	{
		OutputChannel->Uninitialize();
		OutputChannel.Reset();
		OutputCallback.Reset();
	}
}

bool UAjaMediaCapture::HasFinishedProcessing() const
{
	return Super::HasFinishedProcessing() || !OutputChannel;
}

const FMatrix& UAjaMediaCapture::GetRGBToYUVConversionMatrix() const
{
	if (const UAjaMediaOutput* AjaOutput = Cast<UAjaMediaOutput>(MediaOutput))
	{
		switch(AjaOutput->HDROptions.Gamut)
		{
		case EAjaHDRMetadataGamut::Rec709:
			return MediaShaders::RgbToYuvRec709Scaled;
		case EAjaHDRMetadataGamut::Rec2020:
			return MediaShaders::RgbToYuvRec2020Scaled;
		default:
			checkNoEntry();
			return MediaShaders::RgbToYuvRec709Scaled;
		}
	}
	
	return Super::GetRGBToYUVConversionMatrix();
}

void UAjaMediaCapture::WaitForGPU(FRHITexture* InRHITexture)
{
	if (TextureTransfer)
	{
		TextureTransfer->WaitForGPU(InRHITexture);
	}
}


#if WITH_EDITOR
FString UAjaMediaCapture::GetAnalyticsEventPrefix() const
{
	return TEXT("MediaFramework.Aja"); 
}

void UAjaMediaCapture::GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const
{
	Super::GetAnalyticsEventAttributes(EventName, InOutAttributes);

	UAjaMediaOutput* AjaMediaOutput = CastChecked<UAjaMediaOutput>(MediaOutput);

	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Framerate"), AjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.FrameRate.ToPrettyText().ToString()));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Device"), AjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName));

	if (IMediaIOCoreDeviceProvider* DeviceProvider = IMediaIOCoreModule::Get().GetDeviceProvider(FAjaDeviceProvider::GetProviderName()))
	{
		const FString DestStr = FString::Format(TEXT("{0} {1}"), {
			DeviceProvider->GetTransportName(
				AjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.TransportType,
				AjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.QuadTransportType).ToString(),
			AjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.PortIdentifier });
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Destination"), DestStr));
	}

	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Standard"), UEnum::GetValueAsString(AjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.Standard)));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Reference"), UEnum::GetValueAsString(AjaMediaOutput->OutputConfiguration.OutputReference)));
}
#endif

bool UAjaMediaCapture::InitAJA(UAjaMediaOutput* InAjaMediaOutput)
{
	check(InAjaMediaOutput);

	IAjaMediaModule& MediaModule = FModuleManager::LoadModuleChecked<IAjaMediaModule>(TEXT("AjaMedia"));
	if (!MediaModule.CanBeUsed())
	{
		UE_LOGF(LogAjaMediaOutput, Warning, "The AjaMediaCapture can't open MediaOutput '%ls' because Aja card cannot be used. Are you in a Commandlet? You may override this behavior by launching with -ForceAjaUsage", *InAjaMediaOutput->GetName());
		return false;
	}

	// Init general settings
	bWaitForSyncEvent = InAjaMediaOutput->bWaitForSyncEvent;
	bLogDropFrame = InAjaMediaOutput->bLogDropFrame;
	bEncodeTimecodeInTexel = InAjaMediaOutput->bEncodeTimecodeInTexel;
	FrameRate = InAjaMediaOutput->GetRequestedFrameRate();
	PortName = FAjaDeviceProvider().ToText(InAjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection).ToString();

	if (ShouldCaptureRHIResource())
	{
		if (InAjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.Standard != EMediaIOStandardType::Progressive)
		{
			UE_LOGF(LogAjaMediaOutput, Warning, "GPU DMA is not supported with interlaced, defaulting to regular path.");
			TextureTransfer.Reset();
		}
	}

	// Init Device options
	AJA::AJADeviceOptions DeviceOptions(InAjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceIdentifier);

	OutputCallback = MakePimpl<UAjaMediaCapture::FAjaOutputCallback>();
	OutputCallback->Owner = this;

	AJA::AJAVideoFormats::VideoFormatDescriptor Descriptor = AJA::AJAVideoFormats::GetVideoFormat(InAjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.DeviceModeIdentifier);

	// Init Channel options
	AJA::AJAInputOutputChannelOptions ChannelOptions(TEXT("ViewportOutput"), InAjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.PortIdentifier);
	ChannelOptions.CallbackInterface = OutputCallback.Get();
	ChannelOptions.bOutput = true;
	ChannelOptions.NumberOfAudioChannel = static_cast<int32>(InAjaMediaOutput->NumOutputAudioChannels);
	ChannelOptions.SynchronizeChannelIndex = InAjaMediaOutput->OutputConfiguration.ReferencePortIdentifier;
	ChannelOptions.KeyChannelIndex = InAjaMediaOutput->OutputConfiguration.KeyPortIdentifier;
	ChannelOptions.OutputNumberOfBuffers = InAjaMediaOutput->NumberOfAJABuffers;
	ChannelOptions.VideoFormatIndex = InAjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaMode.DeviceModeIdentifier;
	ChannelOptions.bUseAutoCirculating = InAjaMediaOutput->bOutputWithAutoCirculating;
	ChannelOptions.bUseKey = InAjaMediaOutput->OutputConfiguration.OutputType == EMediaIOOutputType::FillAndKey;  // must be RGBA to support Fill+Key
	ChannelOptions.bUseAncillary = false;
	ChannelOptions.bUseAudio = InAjaMediaOutput->bOutputAudio;
	ChannelOptions.bUseVideo = true;
	ChannelOptions.bOutputInterlacedFieldsTimecodeNeedToMatch = InAjaMediaOutput->bInterlacedFieldsTimecodeNeedToMatch && Descriptor.bIsInterlacedStandard && InAjaMediaOutput->TimecodeFormat != EMediaIOTimecodeFormat::None;
	ChannelOptions.bOutputInterlaceAsProgressive = InAjaMediaOutput->bOutputInterlaceAsProgressive;
	ChannelOptions.bOutputInterlaceOnEvenFrames = InAjaMediaOutput->bOutputInterlaceOnEvenFrames;
	ChannelOptions.bDisplayWarningIfDropFrames = bLogDropFrame;
	ChannelOptions.bConvertOutputLevelAToB = InAjaMediaOutput->bOutputIn3GLevelB && Descriptor.bIsVideoFormatA;
	ChannelOptions.TransportType = AjaMediaCaptureUtils::ConvertTransportType(InAjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.TransportType, InAjaMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.QuadTransportType);
	ChannelOptions.PixelFormat = AjaMediaCaptureUtils::ConvertPixelFormat(InAjaMediaOutput->PixelFormat, ChannelOptions.bUseKey);
	ChannelOptions.TimecodeFormat =  AjaMediaCaptureUtils::ConvertTimecode(InAjaMediaOutput->TimecodeFormat);
	ChannelOptions.OutputReferenceType = AjaMediaCaptureUtils::Convert(InAjaMediaOutput->OutputConfiguration.OutputReference);
	ChannelOptions.bUseGPUDMA = ShouldCaptureRHIResource();
	ChannelOptions.bDirectlyWriteAudio = InAjaMediaOutput->bOutputAudioOnAudioThread;
	ChannelOptions.bStopOnTimeout = InAjaMediaOutput->bStopOutputOnCardTimeout;

	ChannelOptions.HDROptions = AjaMediaCaptureUtils::MakeAjaHDRMetadata(InAjaMediaOutput->HDROptions);
	
	bOutputAudio = InAjaMediaOutput->bOutputAudio;
	bDirectlyWriteAudio = InAjaMediaOutput->bOutputAudioOnAudioThread;
	
	if (bOutputAudio)
	{
		if (!CreateAudioOutput(AudioDeviceHandle, InAjaMediaOutput))
		{
			UE_LOGF(LogAjaMediaOutput, Error, "Failed to initialize audio output.");
		}
	}
	
	if (GEngine)
	{
		NumInputChannels = GEngine->GetEngineSubsystem<UMediaIOCoreSubsystem>()->GetNumAudioInputChannels();
		NumOutputChannels = static_cast<int32>(InAjaMediaOutput->NumOutputAudioChannels);
	}
	
	PixelFormat = InAjaMediaOutput->PixelFormat;
	UseKey = ChannelOptions.bUseKey;
	
	OutputChannel = MakePimpl<FAJAOutputChannel>();
	if (!OutputChannel->Initialize(DeviceOptions, ChannelOptions))
	{
		UE_LOGF(LogAjaMediaOutput, Warning, "The AJA output port for '%ls' could not be opened.", *InAjaMediaOutput->GetName());
		OutputChannel.Reset();
		OutputCallback.Reset();
		return false;
	}

	if (bWaitForSyncEvent)
	{
		const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VSync"));
		bool bLockToVsync = CVar->GetValueOnGameThread() != 0;
		if (bLockToVsync)
		{
			UE_LOGF(LogAjaMediaOutput, Warning, "The Engine use VSync and something wants to wait for the sync event. This may break the \"gen-lock\".");
		}

		const bool bIsManualReset = false;
		WakeUpEvent = FPlatformProcess::GetSynchEventFromPool(bIsManualReset);
	}

	if (ShouldCaptureRHIResource())
	{
		UE_LOGF(LogAjaMediaOutput, Display, "Aja capture started using GPU Direct");
	}

	return true;
}

bool UAjaMediaCapture::CreateAudioOutput(const FAudioDeviceHandle& InAudioDeviceHandle, const UAjaMediaOutput* InAjaMediaOutput)
{
	if (GEngine && bOutputAudio && InAjaMediaOutput)
	{
		UMediaIOCoreSubsystem::FCreateAudioOutputArgs Args;
		Args.NumOutputChannels = static_cast<int32>(InAjaMediaOutput->NumOutputAudioChannels);
		Args.TargetFrameRate = FrameRate;
		Args.MaxSampleLatency = Align(InAjaMediaOutput->AudioBufferSize, 4);
		Args.OutputSampleRate = static_cast<uint32>(InAjaMediaOutput->AudioSampleRate);
		Args.AudioDeviceHandle = InAudioDeviceHandle;
		AudioOutput = GEngine->GetEngineSubsystem<UMediaIOCoreSubsystem>()->CreateAudioOutput(Args);
		return AudioOutput.IsValid();
	}
	return false;
}

void UAjaMediaCapture::OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAjaMediaCapture::OnFrameCaptured_RenderingThread);

	FMediaCaptureResourceData InResourceData;
	InResourceData.Buffer = InBuffer;
	InResourceData.Width = Width;
	InResourceData.Height = Height;
	InResourceData.BytesPerRow = BytesPerRow;

	OnFrameCapturedInternal_AnyThread(InBaseData, InUserData, MoveTemp(InResourceData));
}

void UAjaMediaCapture::OnRHIResourceCaptured_RenderingThread(FRHICommandListImmediate& /*RHICmdList*/, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture)
{
	OnRHIResourceCaptured_AnyThread(InBaseData, InUserData, InTexture);
}

void UAjaMediaCapture::LockDMATexture_RenderThread(FTextureRHIRef InTexture)
{
	if (ShouldCaptureRHIResource())
	{
		if (!TexturesToRelease.Contains(InTexture))
		{
			TexturesToRelease.Add(InTexture);

			FRHITexture* Texture = InTexture->GetTexture2D();
			UE::GPUTextureTransfer::FRegisterDMATextureArgs Args;
			Args.RHITexture = Texture;
			
			Args.RHIResourceMemory = nullptr; // = Texture->GetNativeResource(); todo: VulkanTexture->Surface->GetAllocationHandle for Vulkan
			Args.Width = Texture->GetDesc().GetSize().X;
			Args.Height = Texture->GetDesc().GetSize().Y;
			
			if (Args.RHITexture->GetFormat() == EPixelFormat::PF_B8G8R8A8)
			{
				Args.PixelFormat = UE::GPUTextureTransfer::EPixelFormat::PF_8Bit;
				Args.Stride = Args.Width * 4;
			}
			else if (Args.RHITexture->GetFormat() == EPixelFormat::PF_R32G32B32A32_UINT)
			{
				Args.PixelFormat = UE::GPUTextureTransfer::EPixelFormat::PF_10Bit;
				Args.Stride = Args.Width * 16;
			}
			else if (Args.RHITexture->GetFormat() == EPixelFormat::PF_A2B10G10R10)
			{
				// RGB 10 bit can be considered as 8 PF_8 bits by GPUDirect.
				Args.PixelFormat = UE::GPUTextureTransfer::EPixelFormat::PF_8Bit;
				Args.Stride = Args.Width * 4;
			}
			else
			{
				checkf(false, TEXT("Format not supported"));
			}
			
			TextureTransfer->RegisterTexture(Args);
		}

		TextureTransfer->LockTexture(InTexture->GetTexture2D());
	}

}

void UAjaMediaCapture::UnlockDMATexture_RenderThread(FTextureRHIRef InTexture)
{
	if (ShouldCaptureRHIResource())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAjaMediaCapture::OnFrameCaptured_RenderingThread::UnlockDMATexture);
		TextureTransfer->UnlockTexture(InTexture->GetTexture2D());
	}
}

bool UAjaMediaCapture::SupportsAnyThreadCapture() const
{
	return true;
}

void UAjaMediaCapture::OnFrameCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, const FMediaCaptureResourceData& InResourceData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAjaMediaCapture::OnFrameCaptured_AnyThread);
	OnFrameCapturedInternal_AnyThread(InBaseData, InUserData, InResourceData);
}

void UAjaMediaCapture::WaitForSync_AnyThread() const
{
	if (bWaitForSyncEvent)
	{
		if (WakeUpEvent && GetState() != EMediaCaptureState::Error) // Could be shutdown in a middle of a frame 
		{
			WakeUpEvent->Wait();
		}
	}
}

void UAjaMediaCapture::OutputAudio_AnyThread(const AJA::AJAOutputFrameBufferData& FrameBuffer) const
{
	if (bOutputAudio && !bDirectlyWriteAudio)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAjaMediaCapture::OnFrameCaptured_RenderingThread::SetAudio);
		
		// Take a local copy of the audio output in case it is switched from the main thread.
		const TSharedPtr<FMediaIOAudioOutput> LocalAudioOutput = AudioOutput;
		if (!LocalAudioOutput)
		{
			return;
		}
		
		const int32 NumSamplesToPull = FMath::RoundToInt32(48000.f * LocalAudioOutput->NumInputChannels / FrameRate.AsDecimal());
		TArray<int32> AudioSamples = LocalAudioOutput->GetAudioSamples<int32>(NumSamplesToPull);
		OutputChannel->SetAudioFrameData(FrameBuffer, reinterpret_cast<uint8*>(AudioSamples.GetData()), AudioSamples.Num() * sizeof(int32));
	}
}

/* namespace IAJAInputCallbackInterface implementation
// This is called from the AJA thread. There's a lock inside AJA to prevent this object from dying while in this thread.
*****************************************************************************/
void UAjaMediaCapture::FAjaOutputCallback::OnInitializationCompleted(bool bSucceed)
{
	check(Owner);
	if (Owner->GetState() != EMediaCaptureState::Stopped)
	{
		Owner->SetState(bSucceed ? EMediaCaptureState::Capturing : EMediaCaptureState::Error);
	}

	if (Owner->WakeUpEvent)
	{
		Owner->WakeUpEvent->Trigger();
	}
}

bool UAjaMediaCapture::FAjaOutputCallback::OnOutputFrameCopied(const AJA::AJAOutputFrameData& InFrameData)
{
	const uint32 FrameDropCount = InFrameData.FramesDropped;
	if (Owner->bLogDropFrame)
	{
		if (FrameDropCount > LastFrameDropCount)
		{
			PreviousDroppedCount += FrameDropCount - LastFrameDropCount;

			static const int32 NumMaxFrameBeforeWarning = 50;
			if (PreviousDroppedCount % NumMaxFrameBeforeWarning == 0)
			{
				UE_LOGF(LogAjaMediaOutput, Warning, "Loosing frames on AJA output %ls. The current count is %lld.", *Owner->PortName, PreviousDroppedCount);
			}
		}
		else if (PreviousDroppedCount > 0)
		{
			UE_LOGF(LogAjaMediaOutput, Warning, "Lost %lld frames on AJA output %ls. Frame rate may be too slow.", PreviousDroppedCount, *Owner->PortName);
			PreviousDroppedCount = 0;
		}
	}
	LastFrameDropCount = FrameDropCount;

	return true;
}

void UAjaMediaCapture::FAjaOutputCallback::OnOutputFrameStarted()
{
	if (Owner->WakeUpEvent)
	{
		Owner->WakeUpEvent->Trigger();
	}
}

void UAjaMediaCapture::FAjaOutputCallback::OnCompletion(bool bSucceed)
{
	if (!bSucceed)
	{
		Owner->SetState(EMediaCaptureState::Error);
	}

	if (Owner->WakeUpEvent)
	{
		Owner->WakeUpEvent->Trigger();
	}
}

bool UAjaMediaCapture::FAjaOutputCallback::OnRequestInputBuffer(const AJA::AJARequestInputBufferData& RequestBuffer, AJA::AJARequestedInputBufferData& OutRequestedBuffer)
{
	check(false);
	return false;
}

bool UAjaMediaCapture::FAjaOutputCallback::OnInputFrameReceived(const AJA::AJAInputFrameData& InInputFrame, const AJA::AJAAncillaryFrameData& InAncillaryFrame, const AJA::AJAAudioFrameData& AudioFrame, const AJA::AJAVideoFrameData& VideoFrame)
{
	check(false);
	return false;
}

void UAjaMediaCapture::OnRHIResourceCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAjaMediaCapture::OnFrameCaptured_AnyThread);
	
	// Prevent the rendering thread from copying while we are stopping the capture.
	FScopeLock ScopeLock(&CopyingCriticalSection);
	if (OutputChannel)
	{
		const AJA::FTimecode Timecode = AjaMediaCaptureDevice::ConvertToAJATimecode(InBaseData.SourceFrameTimecode, InBaseData.SourceFrameTimecodeFramerate.AsDecimal(), FrameRate.AsDecimal());
		
		AJA::AJAOutputFrameBufferData FrameBuffer;
		FrameBuffer.Timecode = Timecode;
		FrameBuffer.FrameIdentifier = InBaseData.SourceFrameNumberRenderThread;

		bool bSetVideoResult = false;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UAjaMediaCapture::OnFrameCaptured_RenderingThread::SetVideo_GPUDirect);
			bSetVideoResult = OutputChannel->SetVideoFrameData(FrameBuffer, InTexture->GetTexture2D());
		}

		// If the set video call fails, that means we probably didn't find an available frame to write to,
		// so don't pop from the audio buffer since we would lose these samples in the SetAudioFrameData call.
		if (bSetVideoResult)
		{
			OutputAudio_AnyThread(FrameBuffer);
		}
		
		WaitForSync_AnyThread();
		
	}
	else if (GetState() != EMediaCaptureState::Stopped)
	{
		SetState(EMediaCaptureState::Error);
	}
}

void UAjaMediaCapture::OnFrameCapturedInternal_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, const FMediaCaptureResourceData& InResourceData)
{
	// Prevent this thread from copying while we are stopping the capture.
	FScopeLock ScopeLock(&CopyingCriticalSection);
	if (OutputChannel)
	{
		const AJA::FTimecode Timecode = AjaMediaCaptureDevice::ConvertToAJATimecode(InBaseData.SourceFrameTimecode, InBaseData.SourceFrameTimecodeFramerate.AsDecimal(), FrameRate.AsDecimal());
		const AjaMediaCaptureDevice::FAjaMediaEncodeOptions EncodeOptions(InResourceData.Width, InResourceData.Height, PixelFormat, UseKey);

		if (bEncodeTimecodeInTexel)
		{
			const FMediaIOCoreEncodeTime EncodeTime(EncodeOptions.EncodePixelFormat, InResourceData.Buffer, EncodeOptions.Stride, EncodeOptions.TimeEncodeWidth, InResourceData.Height);
			EncodeTime.Render(Timecode.Hours, Timecode.Minutes, Timecode.Seconds, Timecode.Frames);
		}

		AJA::AJAOutputFrameBufferData FrameBuffer;
		FrameBuffer.Timecode = Timecode;
		FrameBuffer.FrameIdentifier = InBaseData.SourceFrameNumber;

		// This will most likely be wrong when using the AnyThread callback 
		FrameBuffer.bEvenFrame = GFrameCounterRenderThread % 2 == 0;

		bool bSetVideoResult = false;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UAjaMediaCapture::OnFrameCaptured::SetVideo);
			bSetVideoResult = OutputChannel->SetVideoFrameData(FrameBuffer, reinterpret_cast<uint8_t*>(InResourceData.Buffer), EncodeOptions.Stride * InResourceData.Height);
		}

		// If the set video call fails, that means we probably didn't find an available frame to write to,
		// so don't pop from the audio buffer since we would lose these samples in the SetAudioFrameData call.
		if (bSetVideoResult)
		{
			OutputAudio_AnyThread(FrameBuffer);
		}

		if (bAjaWriteInputRawDataCmdEnable)
		{
			MediaIOCoreFileWriter::WriteRawFile(EncodeOptions.OutputFilename, reinterpret_cast<uint8*>(InResourceData.Buffer), EncodeOptions.Stride * InResourceData.Height);
			bAjaWriteInputRawDataCmdEnable = false;
		}

		WaitForSync_AnyThread();
	}
	else if (GetState() != EMediaCaptureState::Stopped)
	{
		SetState(EMediaCaptureState::Error);
	}
}

void UAjaMediaCapture::OnAudioBufferReceived_AudioThread(Audio::FDeviceId DeviceId, float* Data, int32 NumSamples) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAjaMediaCapture::AudioBufferReceived);

	if (bOutputAudio && OutputChannel)
	{
		if (ensure(NumInputChannels != 0))
		{
			const TArray<int32> ConvertedSamples = FMediaIOAudioOutput::ConvertAndUpmixBuffer<int32>(MakeArrayView<float>(Data, NumSamples), NumInputChannels, NumOutputChannels);
			OutputChannel->DMAWriteAudio(reinterpret_cast<const uint8*>(ConvertedSamples.GetData()), ConvertedSamples.Num() * sizeof(int32));
		}
	}
}

void UAjaMediaCapture::PostCaptureCleanupImpl()
{
	OutputChannel.Reset();
	OutputCallback.Reset();

	if (WakeUpEvent)
	{
		FPlatformProcess::ReturnSynchEventToPool(WakeUpEvent);
		WakeUpEvent = nullptr;
	}

	if (GEngine)
	{
		UMediaIOCoreSubsystem* SubSystem = GEngine->GetEngineSubsystem<UMediaIOCoreSubsystem>();
		if (SubSystem)
		{
			SubSystem->OnBufferReceived_AudioThread().RemoveAll(this);
		}
	}

	AudioOutput.Reset();

	RestoreViewportTextureAlpha(GetCapturingSceneViewport());
}
