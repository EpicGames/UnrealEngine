// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaMediaFrameGrabberProtocol.h"

#include "AjaMediaCapture.h"
#include "AjaMediaOutput.h"
#include "IAjaMediaOutputModule.h"
#include "MovieSceneCaptureProtocolBase.h"

#define LOCTEXT_NAMESPACE "AjaMediaOutput"


/**
 * UAjaFrameGrabberProtocol
 */

UAjaFrameGrabberProtocol::UAjaFrameGrabberProtocol(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, Information("FrameRate, Resolution, Output Directory and Filename Format options won't be used with AJA output")
	, TransientMediaOutputPtr(nullptr)
	, TransientMediaCapturePtr(nullptr)
{
}

bool UAjaFrameGrabberProtocol::StartCaptureImpl()
{
	if (MediaOutput == nullptr)
	{
		UE_LOGF(LogAjaMediaOutput, Error, "Couldn't start the capture. No Media Output was provided.");
		return false;
	}

	TransientMediaOutputPtr = Cast<UAjaMediaOutput>(MediaOutput.TryLoad());
	if (TransientMediaOutputPtr == nullptr)
	{
		UE_LOGF(LogAjaMediaOutput, Error, "Couldn't start the capture. No Media Output was provided.");
		return false;
	}

	if (TransientMediaOutputPtr->GetRequestedFrameRate() != CaptureHost->GetCaptureFrameRate())
	{
		UE_LOGF(LogAjaMediaOutput, Warning, "AjaMediaOutput %ls FrameRate doesn't match sequence FrameRate.", *TransientMediaOutputPtr->GetName());
		return false;
	}

	TransientMediaCapturePtr = CastChecked<UAjaMediaCapture>(TransientMediaOutputPtr->CreateMediaCapture(), ECastCheckedType::NullAllowed);
	if (TransientMediaCapturePtr)
	{
		bool bResult = TransientMediaCapturePtr->CaptureSceneViewport(InitSettings->SceneViewport, FMediaCaptureOptions());
		if (!bResult)
		{
			UE_LOGF(LogAjaMediaOutput, Error, "Could not initialize the Media Capture.");
			return false;
		}
	}
	else
	{
		UE_LOGF(LogAjaMediaOutput, Error, "Could not create the Media Capture.");
		return false;
	}

	return true;
}

bool UAjaFrameGrabberProtocol::HasFinishedProcessingImpl() const
{
	return TransientMediaCapturePtr == nullptr || TransientMediaCapturePtr->HasFinishedProcessing();
}

void UAjaFrameGrabberProtocol::FinalizeImpl()
{
	if (TransientMediaCapturePtr)
	{
		TransientMediaCapturePtr->StopCapture(true);
		TransientMediaCapturePtr = nullptr;
	}
}

#undef LOCTEXT_NAMESPACE
