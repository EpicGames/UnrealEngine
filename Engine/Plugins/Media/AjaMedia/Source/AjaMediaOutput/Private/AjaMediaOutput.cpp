// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaMediaOutput.h"

#include "AjaDeviceProvider.h"
#include "AJALib.h"
#include "AjaMediaCapture.h"
#include "AjaMediaSettings.h"
#include "IAjaMediaModule.h"
#include "IMediaIOCoreModule.h"
#include "Modules/ModuleManager.h"
#include "UObject/EnterpriseObjectVersion.h"


#define LOCTEXT_NAMESPACE "AjaMediaOutput"


/* UAjaMediaOutput
*****************************************************************************/

UAjaMediaOutput::UAjaMediaOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bOutputWithAutoCirculating(false)
	, TimecodeFormat(EMediaIOTimecodeFormat::LTC)
	, PixelFormat(EAjaMediaOutputPixelFormat::PF_8BIT_YUV)
	, bOutputIn3GLevelB(false)
	, bInvertKeyOutput(false)
	, bOutputAudio(false)
	, NumberOfAJABuffers(2)
	, bInterlacedFieldsTimecodeNeedToMatch(false)
	, bWaitForSyncEvent(false)
	, bLogDropFrame(true)
	, bEncodeTimecodeInTexel(false)
{
}

bool UAjaMediaOutput::Validate(FString& OutFailureReason) const
{
	if (!Super::Validate(OutFailureReason))
	{
		return false;
	}
	
	if (!OutputConfiguration.IsValid())
	{
		OutFailureReason = FString::Printf(TEXT("The Configuration of '%s' is invalid ."), *GetName());
		return false;
	}

	IAjaMediaModule& MediaModule = FModuleManager::LoadModuleChecked<IAjaMediaModule>(TEXT("AjaMedia"));
	if (!MediaModule.IsInitialized())
	{
		OutFailureReason = FString::Printf(TEXT("Can't validate MediaOutput '%s'. The Aja library was not initialized."), *GetName());
		return false;
	}

	if (!MediaModule.CanBeUsed())
	{
		OutFailureReason = FString::Printf(TEXT("Can't validate MediaOutput '%s' because Aja card cannot be used. Are you in a Commandlet? You may override this behavior by launching with -ForceAjaUsage"), *GetName());
		return false;
	}

	AJA::AJADeviceScanner Scanner;
	AJA::AJADeviceScanner::DeviceInfo DeviceInfo;
	if (!Scanner.GetDeviceInfo(OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceIdentifier, DeviceInfo))
	{
		OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that doesn't exist on this machine."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	if (!DeviceInfo.bIsSupported)
	{
		OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that is not supported by the AJA SDK."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	const bool bDeviceHasOutput = DeviceInfo.NumSdiOutput > 0; // || DeviceInfo.NumHdmiOutput > 0 we do not support HDMI output, you should use a normal graphic card.
	if (!bDeviceHasOutput)
	{
		OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that can't do playback."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
		return false;
	}

	if (!DeviceInfo.bCanFrameStore1DoPlayback)
	{
		if (OutputConfiguration.MediaConfiguration.MediaConnection.PortIdentifier == 1)
		{
			OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that can't do playback on port 1."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
			return false;
		}

		if (OutputConfiguration.OutputType == EMediaIOOutputType::FillAndKey && OutputConfiguration.MediaConfiguration.MediaConnection.PortIdentifier == 1)
		{
			OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that can't do playback on port 1."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
			return false;
		}
	}

	if (OutputConfiguration.OutputType == EMediaIOOutputType::FillAndKey)
	{
		// Even if YUV is selected we will later revert to RGBA to allow for Key, make sure we support it.
		if (PixelFormat == EAjaMediaOutputPixelFormat::PF_8BIT_YUV && !DeviceInfo.bSupportPixelFormat8bitARGB)
		{
			OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that doesn't support the 8bit ARGB pixel format."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
			return false;
		}
		if (PixelFormat == EAjaMediaOutputPixelFormat::PF_10BIT_YUV && !DeviceInfo.bSupportPixelFormat10bitRGB)
		{
			OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that doesn't support the 10bit RGB pixel format."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
			return false;
		}
	}
	else
	{
		if (PixelFormat == EAjaMediaOutputPixelFormat::PF_8BIT_YUV && !DeviceInfo.bSupportPixelFormat8bitYCBCR)
		{
			OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that doesn't support the 8bit YUV pixel format."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
			return false;
		}
		if (PixelFormat == EAjaMediaOutputPixelFormat::PF_10BIT_YUV && !DeviceInfo.bSupportPixelFormat10bitYCBCR)
		{
			OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that doesn't support the 10bit YUV pixel format."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
			return false;
		}
	}

	if (bOutputIn3GLevelB)
	{
		if (!DeviceInfo.bCanDo3GLevelConversion)
		{
			OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' use the device '%s' that doesn't support the 3G level conversion."), *GetName(), *OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
			return false;
		}
		AJA::AJAVideoFormats::VideoFormatDescriptor Descriptor = AJA::AJAVideoFormats::GetVideoFormat(OutputConfiguration.MediaConfiguration.MediaMode.DeviceModeIdentifier);
		if (!Descriptor.bIsVideoFormatA)
		{
			OutFailureReason = FString::Printf(TEXT("The MediaOutput '%s' wants level A to level B conversion but it's not supported by the format."), *GetName());
			return false;
		}
	}

	return true;
}

FFrameRate UAjaMediaOutput::GetRequestedFrameRate() const
{
	return OutputConfiguration.MediaConfiguration.MediaMode.FrameRate;
}

FIntPoint UAjaMediaOutput::GetRequestedSize() const
{
	return OutputConfiguration.MediaConfiguration.MediaMode.Resolution;
}

EPixelFormat UAjaMediaOutput::GetRequestedPixelFormat() const
{
	EPixelFormat Result = EPixelFormat::PF_A2B10G10R10;
	switch (PixelFormat)
	{
	case EAjaMediaOutputPixelFormat::PF_8BIT_YUV:
		Result = EPixelFormat::PF_B8G8R8A8;
		break;
	case EAjaMediaOutputPixelFormat::PF_10BIT_YUV:
		Result = EPixelFormat::PF_A2B10G10R10;
		break;
	}
	return Result;
}

EMediaCaptureConversionOperation UAjaMediaOutput::GetConversionOperation(EMediaCaptureSourceType InSourceType) const
{
	EMediaCaptureConversionOperation Result = EMediaCaptureConversionOperation::NONE;

	switch (PixelFormat)
	{
	case EAjaMediaOutputPixelFormat::PF_8BIT_YUV:
		if (OutputConfiguration.OutputType == EMediaIOOutputType::Fill)
		{
			Result = EMediaCaptureConversionOperation::RGBA8_TO_YUV_8BIT;
		}
		else if (OutputConfiguration.OutputType == EMediaIOOutputType::FillAndKey && bInvertKeyOutput)
		{
			Result = EMediaCaptureConversionOperation::INVERT_ALPHA;
		}
		else
		{
			Result = EMediaCaptureConversionOperation::NONE;
		}
		break;
	case EAjaMediaOutputPixelFormat::PF_10BIT_YUV:
		if (OutputConfiguration.OutputType == EMediaIOOutputType::Fill)
		{
			Result = EMediaCaptureConversionOperation::RGB10_TO_YUVv210_10BIT;
		}
		else if (OutputConfiguration.OutputType == EMediaIOOutputType::FillAndKey && bInvertKeyOutput)
		{
			Result = EMediaCaptureConversionOperation::INVERT_ALPHA;
		}
		else
		{
			Result = EMediaCaptureConversionOperation::NONE;
		}
		break;
	}
	return Result;
}

#if WITH_EDITOR
FString UAjaMediaOutput::GetDescriptionString() const
{
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(FAjaDeviceProvider::GetProviderName());
	if (DeviceProviderPtr)
	{
		return DeviceProviderPtr->ToText(OutputConfiguration).ToString(); 
	}

	return Super::GetDescriptionString();
}

void UAjaMediaOutput::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(FAjaDeviceProvider::GetProviderName());
	if (!DeviceProviderPtr)
	{
		return;
	}
	
	const FText ConfigHeader = LOCTEXT("MediaConfigurationHeader", "Media Configuration");

	const FText SourceText = FText::Format(LOCTEXT("TransportFormat", "{0} {1}"),
		DeviceProviderPtr->GetTransportName(OutputConfiguration.MediaConfiguration.MediaConnection.TransportType, OutputConfiguration.MediaConfiguration.MediaConnection.QuadTransportType),
		FText::AsNumber(OutputConfiguration.MediaConfiguration.MediaConnection.PortIdentifier));
	
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("DeviceLabel", "Device"), FText::FromName(OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName)));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("SourceLabel", "Source"), SourceText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("ResolutionLabel", "Resolution"), OutputConfiguration.MediaConfiguration.MediaMode.GetModeName()));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("StandardLabel", "Standard"), UEnum::GetDisplayValueAsText(OutputConfiguration.MediaConfiguration.MediaMode.Standard)));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("FramerateLabel", "Framerate"), OutputConfiguration.MediaConfiguration.MediaMode.FrameRate.ToPrettyText()));
}
#endif // WITH_EDITOR

UMediaCapture* UAjaMediaOutput::CreateMediaCaptureImpl()
{
	UMediaCapture* Result = NewObject<UAjaMediaCapture>();
	if (Result)
	{
		Result->SetMediaOutput(this);
	}
	return Result;
}

#if WITH_EDITOR
bool UAjaMediaOutput::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAjaMediaOutput, bEncodeTimecodeInTexel))
	{
		return TimecodeFormat != EMediaIOTimecodeFormat::None;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAjaMediaOutput, bOutputIn3GLevelB))
	{
		bool bValid = false;
		if (OutputConfiguration.IsValid())
		{
			AJA::AJAVideoFormats::VideoFormatDescriptor Descriptor = AJA::AJAVideoFormats::GetVideoFormat(OutputConfiguration.MediaConfiguration.MediaMode.DeviceModeIdentifier);
			bValid = Descriptor.bIsVideoFormatA;
		}
		return bValid;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAjaMediaOutput, bInvertKeyOutput))
	{
		return (OutputConfiguration.OutputType == EMediaIOOutputType::FillAndKey);
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAjaMediaOutput, bInterlacedFieldsTimecodeNeedToMatch))
	{
		bool bValid = false;
		if (OutputConfiguration.IsValid() && TimecodeFormat != EMediaIOTimecodeFormat::None)
		{
			AJA::AJAVideoFormats::VideoFormatDescriptor Descriptor = AJA::AJAVideoFormats::GetVideoFormat(OutputConfiguration.MediaConfiguration.MediaMode.DeviceModeIdentifier);
			bValid = Descriptor.bIsInterlacedStandard;
		}
		return bValid;
	}

	return true;
}

void UAjaMediaOutput::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAjaMediaOutput, TimecodeFormat))
	{
		if (TimecodeFormat == EMediaIOTimecodeFormat::None)
		{
			bEncodeTimecodeInTexel = false;
			bInterlacedFieldsTimecodeNeedToMatch = false;
		}
	}

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAjaMediaOutput, OutputConfiguration))
	{
		if (bOutputIn3GLevelB)
		{
			bOutputIn3GLevelB = false;
			if (OutputConfiguration.IsValid())
			{
				AJA::AJAVideoFormats::VideoFormatDescriptor Descriptor = AJA::AJAVideoFormats::GetVideoFormat(OutputConfiguration.MediaConfiguration.MediaMode.DeviceModeIdentifier);
				bOutputIn3GLevelB = Descriptor.bIsVideoFormatA;
			}
		}

		if (OutputConfiguration.OutputType == EMediaIOOutputType::Fill)
		{
			bInvertKeyOutput = false;
		}

		if (bInterlacedFieldsTimecodeNeedToMatch)
		{
			bInterlacedFieldsTimecodeNeedToMatch = false;
			if (OutputConfiguration.IsValid() && TimecodeFormat != EMediaIOTimecodeFormat::None)
			{
				AJA::AJAVideoFormats::VideoFormatDescriptor Descriptor = AJA::AJAVideoFormats::GetVideoFormat(OutputConfiguration.MediaConfiguration.MediaMode.DeviceModeIdentifier);
				bInterlacedFieldsTimecodeNeedToMatch = Descriptor.bIsInterlacedStandard;
			}
		}
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif //WITH_EDITOR

#undef LOCTEXT_NAMESPACE
