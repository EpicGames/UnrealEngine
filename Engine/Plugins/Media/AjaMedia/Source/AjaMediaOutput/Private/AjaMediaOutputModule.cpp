// Copyright Epic Games, Inc. All Rights Reserved.

#include "AjaMediaOutputModule.h"

#define LOCTEXT_NAMESPACE "AjaMediaOutput"

DEFINE_LOG_CATEGORY(LogAjaMediaOutput);

void FAjaMediaOutputModule::StartupModule()
{
}

void FAjaMediaOutputModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FAjaMediaOutputModule, AjaMediaOutput )

#undef LOCTEXT_NAMESPACE
