// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAjaMediaOutputModule.h"

#include "Modules/ModuleManager.h"

class FAjaMediaOutputModule : public IAjaMediaOutputModule
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};