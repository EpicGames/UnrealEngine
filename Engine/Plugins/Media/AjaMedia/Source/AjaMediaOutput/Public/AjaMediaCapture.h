// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaCapture.h"
#include "AjaMediaOutput.h"
#include "DSP/BufferVectorOperations.h"
#include "HAL/CriticalSection.h"
#include "MediaIOCoreEncodeTime.h"
#include "Misc/FrameRate.h"

#include "AjaMediaCapture.generated.h"


class FEvent;
class UAjaMediaOutput;

namespace AJA
{
	struct AJAOutputFrameBufferData;
}

namespace UE::GPUTextureTransfer
{
	class ITextureTransfer;
}

/**
 * Output Media for AJA streams.
 * The output format could be any of EAjaMediaOutputPixelFormat.
 */
UCLASS(BlueprintType)
class AJAMEDIAOUTPUT_API UAjaMediaCapture : public UMediaCapture
{
	GENERATED_BODY()

public:
	UAjaMediaCapture();
	~UAjaMediaCapture();

	//~ UMediaCapture interface
	virtual bool HasFinishedProcessing() const override;
protected:
	virtual bool ValidateMediaOutput() const override;
	virtual bool InitializeCapture() override;
	virtual bool PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport) override;
	virtual bool UpdateSceneViewportImpl(TSharedPtr<FSceneViewport>& InSceneViewport) override;
	virtual bool UpdateRenderTargetImpl(UTextureRenderTarget2D* InRenderTarget) override;
	virtual bool UpdateMediaTextureImpl(UMediaTexture* InMediaTexture) override;
	virtual bool UpdateAudioDeviceImpl(const FAudioDeviceHandle& InAudioDeviceHandle) override;
	virtual void StopCaptureImpl(bool bAllowPendingFrameToBeProcess) override;
	virtual void PostCaptureCleanupImpl() override;
	virtual bool SupportsStopCapture_Future() const override { return true; }
	virtual TFuture<void> StopCapture_Future() override;
	virtual bool ShouldCaptureRHIResource() const override;
	virtual bool ShouldCaptureThisFrame(const FCaptureBaseData& InBaseData) const override;
	
	virtual void OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow) override;
	virtual void OnRHIResourceCaptured_RenderingThread(FRHICommandListImmediate& RHICmdList, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture) override;
	virtual void OnRHIResourceCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture) override;
	virtual void LockDMATexture_RenderThread(FTextureRHIRef InTexture) override;
	virtual void UnlockDMATexture_RenderThread(FTextureRHIRef InTexture) override;
	virtual void OnFrameCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, const FMediaCaptureResourceData& InResourceData) override;
	virtual bool SupportsAnyThreadCapture() const override;
	virtual const FMatrix& GetRGBToYUVConversionMatrix() const override;
	virtual void WaitForGPU(FRHITexture* InRHITexture) override;

#if WITH_EDITOR
	virtual FString GetAnalyticsEventPrefix() const override;
	virtual void GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const override;
#endif

private:
	struct FAjaOutputCallback;
	friend FAjaOutputCallback;
	struct FAJAOutputChannel;

private:
	bool InitAJA(UAjaMediaOutput* InMediaOutput);
	bool CreateAudioOutput(const FAudioDeviceHandle& InAudioDeviceHandle, const UAjaMediaOutput* InAjaMediaOutput);
	void WaitForSync_AnyThread() const;
	void OutputAudio_AnyThread(const AJA::AJAOutputFrameBufferData& FrameBuffer) const;
	void ApplyViewportTextureAlpha(TSharedPtr<FSceneViewport> InSceneViewport);
	void RestoreViewportTextureAlpha(TSharedPtr<FSceneViewport> InSceneViewport);
	bool CleanupPreEditorExit();
	void OnEnginePreExit();
	void OnFrameCapturedInternal_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, const FMediaCaptureResourceData& InResourceData);
	void OnAudioBufferReceived_AudioThread(Audio::FDeviceId DeviceId, float* Data, int32 NumSamples) const;

	struct FAudioBuffer
	{
		Audio::FAlignedFloatBuffer Data;
		int32 NumChannels = 0;
	};
	
private:
	/** AJA Port for outputting */
	TPimplPtr<FAJAOutputChannel> OutputChannel;
	TPimplPtr<FAjaOutputCallback> OutputCallback;

	/** Name of this output port */
	FString PortName;

	/** Option from MediaOutput */
	bool bWaitForSyncEvent;
	bool bLogDropFrame;
	bool bEncodeTimecodeInTexel;
	EAjaMediaOutputPixelFormat PixelFormat;
	bool UseKey;

	/** Saved IgnoreTextureAlpha flag from viewport */
	bool bSavedIgnoreTextureAlpha;
	bool bIgnoreTextureAlphaChanged;

	/** Selected FrameRate of this output */
	FFrameRate FrameRate;

	/** Critical section for synchronizing access to the OutputChannel */
	FCriticalSection CopyingCriticalSection;

	/** Event to wakeup When waiting for sync */
	FEvent* WakeUpEvent;

	/** Holds an audio output that captures audio from the engine. */
	TSharedPtr<class FMediaIOAudioOutput> AudioOutput;

	/** Number of audio channels output to the AJA card.  */
	int32 NumOutputChannels = 0;

	/** Number of audio channels the engine is rendering to. */
	int32 NumInputChannels =0;

	bool bOutputAudio = false;

	/** Whether to write audio on the audio thread instead of collecting samples and outputting on the render thread.. */
	bool bDirectlyWriteAudio = false;
	
	/** Textures to release when the capture has stopped. Must be released after GPUTextureTransfer textures have been unregistered. */
	TArray<FTextureRHIRef> TexturesToRelease;

	/** Handle for the delegate used to clean up AJA on editor shutdown. */
	FDelegateHandle CanCloseEditorDelegateHandle;

	mutable int32 AudioSamplesSentLastFrame = 0;

	TSharedPtr<UE::GPUTextureTransfer::ITextureTransfer> TextureTransfer;
};
