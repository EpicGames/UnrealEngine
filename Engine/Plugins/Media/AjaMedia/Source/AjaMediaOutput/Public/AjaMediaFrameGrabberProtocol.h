// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Protocols/FrameGrabberProtocol.h"
#include "UObject/StrongObjectPtr.h"

#include "AjaMediaFrameGrabberProtocol.generated.h"

class UAjaMediaCapture;
class UAjaMediaOutput;

UCLASS(meta=(DisplayName="AJA Output", CommandLineID="AJAOutput"))
class AJAMEDIAOUTPUT_API UAjaFrameGrabberProtocol : public UMovieSceneImageCaptureProtocolBase
{
public:
	GENERATED_BODY()

	UAjaFrameGrabberProtocol(const FObjectInitializer& ObjInit);

	/** ~UMovieSceneCaptureProtocolBase implementation */
	virtual bool StartCaptureImpl() override;
	virtual bool HasFinishedProcessingImpl() const override;
	virtual void FinalizeImpl() override;
	virtual bool CanWriteToFileImpl(const TCHAR* InFilename, bool bOverwriteExisting) const { return false; }
	/** ~End UMovieSceneCaptureProtocolBase implementation */

public:

	/** AJA Setting to use for the FrameGrabberProtocol */
	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category=AJA, meta=(AllowedClasses="/Script/AjaMediaOutput.AjaMediaOutput"))
	FSoftObjectPath MediaOutput;

	/** States unused options for AJAFrameGrabberProtocolSettings */
	UPROPERTY(VisibleAnywhere, Transient, Category=AJA)
	FString Information;

private:

	/** Transient media output pointer to keep the media output alive while this protocol is in use */
	UPROPERTY(Transient)
	TObjectPtr<UAjaMediaOutput> TransientMediaOutputPtr;

	/** Transient media capture pointer that will capture the viewport */
	UPROPERTY(Transient)
	TObjectPtr<UAjaMediaCapture> TransientMediaCapturePtr;
};
