// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaOutput.h"

#include "AjaMediaDefinitions.h"
#include "MediaIOCoreDefinitions.h"

#include "AjaMediaOutput.generated.h"

/**
 * Native data format.
 */
UENUM()
enum class EAjaMediaOutputPixelFormat : uint8
{
	PF_8BIT_YUV UMETA(DisplayName = "8bit YUV"),
	PF_10BIT_YUV UMETA(DisplayName = "10bit YUV"),
};

UENUM()
enum class EAjaMediaOutputAudioSampleRate : uint32
{
	SR_48k = 48000 UMETA(DisplayName = "48 kHz")
};

/**
 * Number of output audio channels
 */
UENUM()
enum class EAjaMediaOutputChannelConfiguration : uint8
{
	CH_6 =   6 UMETA(DisplayName = "6 Channels"),
	CH_8 =   8 UMETA(DisplayName = "8 Channels"),
	CH_16 = 16 UMETA(DisplayName = "16 Channels")
};

/**
 * Output information for an aja media capture.
 * @note	'Frame Buffer Pixel Format' must be set to at least 8 bits of alpha to enabled the Key.
 * @note	'Enable alpha channel support in post-processing' must be set to 'Allow through tonemapper' to enabled the Key.
 */
UCLASS(BlueprintType, PrioritizeCategories=(AJA, Output, Synchronization, Debug), meta=(MediaIOCustomLayout="AJA"))
class AJAMEDIAOUTPUT_API UAjaMediaOutput : public UMediaOutput
{
	GENERATED_UCLASS_BODY()

public:

	/** The device, port and video settings that correspond to the output. */
	UPROPERTY(EditAnywhere, Category="AJA", meta=(DisplayName="Configuration"))
	FMediaIOOutputConfiguration OutputConfiguration;

	/**
	 * If this is disabled, output will keep going even if we hit a timeout when waiting for a vertical interrupt from the card.
	 */
	UPROPERTY(EditAnywhere, Category = "AJA")
	bool bStopOutputOnCardTimeout = true;

public:
	/**
	 * The output of the Audio, Ancillary and/or video will be perform at the same time.
	 * This may decrease transfer performance but each the data will be sync in relation with each other.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Output")
	bool bOutputWithAutoCirculating;

	/** Whether to embed the Engine's timecode to the output frame. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Output")
	EMediaIOTimecodeFormat TimecodeFormat;

	/** Native data format internally used by the device before being converted to SDI/HDMI signal. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Output")
	EAjaMediaOutputPixelFormat PixelFormat;

	/** If the video format is compatible with 3G Level A, do the conversion to output in LevelB. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Output")
	bool bOutputIn3GLevelB;

	/** Invert Key Output */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Output")
	bool bInvertKeyOutput;

	/** Whether to capture and output audio from the engine. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output")
	bool bOutputAudio;
	
	/** Size of the buffer that holds rendered audio samples, a bigger buffer will produce a more stable output signal but will introduce more delay. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output")
	int32 AudioBufferSize = 5*1024;
	
	/** Number of audio channels used when output audio on the card. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output")
	EAjaMediaOutputChannelConfiguration NumOutputAudioChannels = EAjaMediaOutputChannelConfiguration::CH_8;
	
	/** Audio output sample rate. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output")
	EAjaMediaOutputAudioSampleRate AudioSampleRate = EAjaMediaOutputAudioSampleRate::SR_48k;

	/** Whether to output audio on the audio thread instead of the rendering thread. Only available in Ping Pong mode. (Experimental) */
    UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Output", meta = (EditCondition = "bOutputWithAutoCirculating == false"))
	bool bOutputAudioOnAudioThread = false;

	/**
	 * Number of frame used to transfer from the system memory to the AJA card.
	 * A smaller number is most likely to cause missed frame.
	 * A bigger number is most likely to increase latency.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Output", meta=(ClampMin=1, ClampMax=4))
	int32 NumberOfAJABuffers;

	/**
	 * Only make sense in interlaced mode.
	 * When creating a new Frame the 2 fields need to have the same timecode value.
	 * The Engine's need a TimecodeProvider (or the default system clock) that is in sync with the generated fields.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Output")
	bool bInterlacedFieldsTimecodeNeedToMatch;

	/**
	 * If this is enabled when outputting media in interlace format, frames will be treated as progressive and will be outputted as-is. (Experimental)
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Output")
	bool bOutputInterlaceAsProgressive = false;

	/**
	 * Whether to only schedule interlace frames if we land on an even timecode. Useful when you have an interlace input
	 * and want to ensure that we interlace the original frames. (Experimental)
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Output")
	bool bOutputInterlaceOnEvenFrames = false;

	/**
	 * HDR Metadata of the video signal.
	 */
	UPROPERTY(EditAnywhere, Category = "Output", DisplayName="HDR")
	FAjaMediaHDROptions HDROptions;

	/** Try to maintain a the engine "Genlock" with the VSync signal. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Synchronization")
	bool bWaitForSyncEvent;

public:
	/** Log a warning when there's a drop frame. */
	UPROPERTY(EditAnywhere, Category="Debug")
	bool bLogDropFrame;

	/** Burn Frame Timecode on the output without any frame number clipping. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Debug", meta=(DisplayName="Burn Frame Timecode"))
	bool bEncodeTimecodeInTexel;

public:
	virtual bool Validate(FString& FailureReason) const override;

	FFrameRate GetRequestedFrameRate() const;
	virtual FIntPoint GetRequestedSize() const override;
	virtual EPixelFormat GetRequestedPixelFormat() const override;
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override;

#if WITH_EDITOR
	virtual FString GetDescriptionString() const override;
	virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const override;
#endif //WITH_EDITOR
	
protected:
	virtual UMediaCapture* CreateMediaCaptureImpl() override;

public:
	//~ UObject interface
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject interface
};
